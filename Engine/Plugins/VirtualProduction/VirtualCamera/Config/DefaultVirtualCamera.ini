[CoreRedirects]
+PackageRedirects=(OldName="/VirtualCamera/OperatorPanel...", NewName="/VirtualCamera/Deprecated/OperatorPanel", MatchWildcard=true)
+PackageRedirects=(OldName="/VirtualCamera/TestMaps...", NewName="/VirtualCamera/Deprecated/TestMaps", MatchWildcard=true)
+PackageRedirects=(OldName="/VirtualCamera/V2...", NewName="/VirtualCamera/Deprecated/V2", MatchWildcard=true)
+PackageRedirects=(OldName="/VirtualCamera/VCamCore...", NewName="/VirtualCamera/Deprecated/VCamCore", MatchWildcard=true)