// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchies/BaseModifierGroup.h"

#include "Misc/Guid.h"

void UBaseModifierGroup::PostInitProperties()
{
	NodeName = FName(*FGuid::NewGuid().ToString());
	UObject::PostInitProperties();
}