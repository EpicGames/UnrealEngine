// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Queue.h"

namespace UE::VCamExtensions::HierarchyUtils
{
	enum class EBreakBehavior
	{
		Continue,
		Break
	};

	
	/**
	 * Utility function for iterating a tree like structure.
	 *
	 * ProcessRelationShipFunc = EBreakBehavior(TNodeType& CurrentGroup, TNodeType* Parent)
	 * GetChildrenFunc = TArray<TNodeType*>(TNodeType& CurrentGroup)
	 */
	template<typename TNodeType, typename TProcessRelationShipFunc, typename TGetChildrenFunc>
	void ForEachGroup(TNodeType& RootNode, TProcessRelationShipFunc ProcessRelationShipCallback, TGetChildrenFunc GetChildrenFunc)
	{
		TQueue<TPair<TNodeType*, TNodeType*>> Queue;
		Queue.Enqueue({ &RootNode, nullptr });

		TPair<TNodeType*, TNodeType*> Current;
		while (Queue.Dequeue(Current))
		{
			TNodeType* CurrentNode = Current.Key;
			TNodeType* Parent = Current.Value;
		
			const HierarchyUtils::EBreakBehavior BreakBehavior = ProcessRelationShipCallback(*CurrentNode, Parent);
			if (BreakBehavior == EBreakBehavior::Break)
			{
				return;
			}
		
			for (TNodeType* Child : GetChildrenFunc(*CurrentNode))
			{
				if (Child)
				{
					Queue.Enqueue({ Child, CurrentNode });
				}
			}
		}
	}
};
