// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchies/ModifierHierarchyAsset.h"

#include "Hierarchies/ModifierHierarchyRules.h"

FName UModifierHierarchyAsset::GetRootNode() const
{
	return Rules ? Rules->GetRootNode() : NAME_None;
}

bool UModifierHierarchyAsset::GetParentNode(FName ChildNode, FName& ParentNode) const
{
	return Rules && Rules->GetParentNode(ChildNode, ParentNode);
}

TSet<FName> UModifierHierarchyAsset::GetChildNodes(FName ParentGroup) const
{
	return Rules ? Rules->GetChildNodes(ParentGroup) : TSet<FName>{};
}

UVCamModifier* UModifierHierarchyAsset::GetModifierInNode(UVCamComponent* Component, FName GroupName) const
{
	return Rules ? Rules->GetModifierInNode(Component, GroupName) : nullptr;
}

bool UModifierHierarchyAsset::GetConnectionPointTargetForNode(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connection) const
{
	if (Rules)
	{
		return Rules->GetConnectionPointTargetForNode(GroupName, Component, Connection);
	}
	return false;
}

TSet<FName> UModifierHierarchyAsset::GetGroupsContainingModifier(UVCamModifier* Modifier) const
{
	if (Rules)
	{
		return Rules->GetNodesContainingModifier(Modifier);
	}
	return {};
}
