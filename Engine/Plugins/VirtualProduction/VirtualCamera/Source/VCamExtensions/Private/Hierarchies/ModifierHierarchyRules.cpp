// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchies/ModifierHierarchyRules.h"

FName UModifierHierarchyRules::GetRootNode_Implementation() const
{
	unimplemented();
	return {};
}

bool UModifierHierarchyRules::GetParentNode_Implementation(FName ChildNode, FName& ParentNode) const
{
	return false;
}

TSet<FName> UModifierHierarchyRules::GetChildNodes_Implementation(FName Node) const
{
	unimplemented();
	return {};
}

UVCamModifier* UModifierHierarchyRules::GetModifierInNode_Implementation(UVCamComponent* Component, FName NodeName) const
{
	unimplemented();
	return {};
}

bool UModifierHierarchyRules::GetConnectionPointTargetForNode_Implementation(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connection) const
{
	unimplemented();
	return false;
}

TSet<FName> UModifierHierarchyRules::GetNodesContainingModifier_Implementation(UVCamModifier* Modifier) const
{
	unimplemented();
	return {};
}
