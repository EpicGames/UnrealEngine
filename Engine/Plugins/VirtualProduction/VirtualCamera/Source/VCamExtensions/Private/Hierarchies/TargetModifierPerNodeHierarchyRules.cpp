// Copyright Epic Games, Inc. All Rights Reserved.

#include "Hierarchies/TargetModifierPerNodeHierarchyRules.h"

#include "HierarchyUtils.h"
#include "VCamComponent.h"
#include "VCamExtensionsLog.h"

UTargetModifierPerNodeHierarchyRules::UTargetModifierPerNodeHierarchyRules()
{
	Root = CreateDefaultSubobject<USingleModifierPerNodeWithTargetSettings>(TEXT("Root"));
}

FName UTargetModifierPerNodeHierarchyRules::GetRootNode_Implementation() const
{
	return ensure(Root) ? Root->NodeName : NAME_None;
}

bool UTargetModifierPerNodeHierarchyRules::GetParentNode_Implementation(FName ChildGroup, FName& ParentGroup) const
{
	bool bFound = false;
	ForEachGroup([ChildGroup, &ParentGroup, &bFound](USingleModifierPerNodeWithTargetSettings& Group, USingleModifierPerNodeWithTargetSettings* Parent)
	{
		if (Group.NodeName == ChildGroup)
		{
			ParentGroup = Parent->NodeName;
			bFound = true;
			return EBreakBehavior::Break;
		}
		return EBreakBehavior::Continue;
	});
	return bFound;
}

TSet<FName> UTargetModifierPerNodeHierarchyRules::GetChildNodes_Implementation(FName ParentGroup) const
{
	if (const USingleModifierPerNodeWithTargetSettings* Group = FindNodeByName(ParentGroup))
	{
		TSet<FName> Result;
		Algo::TransformIf(Group->ChildElements, Result, [](const TObjectPtr<USingleModifierPerNodeWithTargetSettings>& Child){ return Child != nullptr; }, [](const TObjectPtr<USingleModifierPerNodeWithTargetSettings>& Child){ return Child->NodeName; });
		return Result;
	}
	return {};
}

UVCamModifier* UTargetModifierPerNodeHierarchyRules::GetModifierInNode_Implementation(UVCamComponent* Component, FName GroupName) const
{
	if (const USingleModifierPerNodeWithTargetSettings* Group = FindNodeByName(GroupName))
	{
		UVCamModifier* Modifier = Component->GetModifierByName(Group->TargetSettings.TargetModifierName);
		return Modifier;
	}

	return {};
}

bool UTargetModifierPerNodeHierarchyRules::GetConnectionPointTargetForNode_Implementation(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connections) const
{
	USingleModifierPerNodeWithTargetSettings* GroupNode = FindNodeByName(GroupName);
	if (!GroupNode)
	{
		return false;
	}

	const FVCamConnectionTargetSettings& TargetSettings = GroupNode->TargetSettings;
	UVCamModifier* Modifier = Component->GetModifierByName(TargetSettings.TargetModifierName);
	if (!Modifier)
	{
		return false;
	}
	
	Connections = { Modifier, TargetSettings.TargetConnectionPoint };
	return true;
}

TSet<FName> UTargetModifierPerNodeHierarchyRules::GetNodesContainingModifier_Implementation(UVCamModifier* Modifier) const
{
	TSet<FName> Groups;
	Algo::Transform(GetNodeForModifier(Modifier), Groups, [](USingleModifierPerNodeWithTargetSettings* Node){ return Node->NodeName; });
	return Groups;
}

TSet<USingleModifierPerNodeWithTargetSettings*> UTargetModifierPerNodeHierarchyRules::GetNodeForModifier(UVCamModifier* Modifier) const
{
	if (!Modifier)
	{
		return {};
	}

	TSet<USingleModifierPerNodeWithTargetSettings*> FoundGroups;
	ForEachGroup([&FoundGroups, SearchName = Modifier->GetStackEntryName()](USingleModifierPerNodeWithTargetSettings& CurrentGroup, USingleModifierPerNodeWithTargetSettings* Parent)
	{
		if (CurrentGroup.TargetSettings.TargetModifierName == SearchName)
		{
			FoundGroups.Add(&CurrentGroup);
		}
		return EBreakBehavior::Continue;
	});
	return FoundGroups;
}

USingleModifierPerNodeWithTargetSettings* UTargetModifierPerNodeHierarchyRules::FindNodeByName(FName GroupName) const
{
	USingleModifierPerNodeWithTargetSettings* FoundGroup = nullptr;
	ForEachGroup([GroupName, &FoundGroup](USingleModifierPerNodeWithTargetSettings& Group, USingleModifierPerNodeWithTargetSettings* Parent)
	{
		if (Group.NodeName == GroupName)
		{
			FoundGroup = &Group;
			return EBreakBehavior::Break;
		}
		return EBreakBehavior::Continue;
	});
	UE_CLOGF(FoundGroup == nullptr, LogVCamExtensions, Warning, "GroupName %ls not found (%ls)", *GroupName.ToString(), *GetPathName());
	return FoundGroup;
}

void UTargetModifierPerNodeHierarchyRules::ForEachGroup(TFunctionRef<EBreakBehavior(USingleModifierPerNodeWithTargetSettings& CurrentGroup, USingleModifierPerNodeWithTargetSettings* Parent)> Callback) const
{
	if (!ensure(Root))
	{
		return;
	}

	using namespace UE::VCamExtensions;
	HierarchyUtils::ForEachGroup(
		*Root,
		[Callback](USingleModifierPerNodeWithTargetSettings& CurrentGroup, USingleModifierPerNodeWithTargetSettings* Parent)
		{
			return Callback(CurrentGroup, Parent) == EBreakBehavior::Continue ? HierarchyUtils::EBreakBehavior::Continue : HierarchyUtils::EBreakBehavior::Break;
		},
		[](USingleModifierPerNodeWithTargetSettings& CurrentGroup){ return CurrentGroup.ChildElements; }
		);
}
