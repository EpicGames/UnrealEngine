// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ClassBasedWidgetStyleDefinitions.h"

#include "VCamComponent.h"
#include "Algo/ForEach.h"

namespace UE::VCamExtensions::Private
{
	static TArray<UWidgetStyleData*> AccumulateMetaDataRecursively(
		const TMap<TSubclassOf<UVCamModifier>, FPerModifierClassWidgetSytleData>& Config,
		UVCamModifier* Modifier,
		TFunctionRef<const FWidgetStyleDataConfig*(const FPerModifierClassWidgetSytleData& ClassConfig)> RetrieveConfig)
	{
		TArray<UWidgetStyleData*> Result;
		if (!Modifier)
		{
			return Result;
		}

		for (UClass* ModifierClass = Modifier->GetClass(); ModifierClass != UVCamModifier::StaticClass()->GetSuperClass(); ModifierClass = ModifierClass->GetSuperClass())
		{
			const FPerModifierClassWidgetSytleData* ClassConfig = Config.Find(ModifierClass);
			if (!ClassConfig)
			{
				continue;
			}
			
			const FWidgetStyleDataConfig* MetaDataConfig = RetrieveConfig(*ClassConfig);
			if (!MetaDataConfig)
			{
				continue;
			}

			Algo::ForEach(MetaDataConfig->ModifierMetaData, [&Result](const TObjectPtr<UWidgetStyleData>& MetaData) { Result.Add(MetaData); });
			if (!MetaDataConfig->bInherit)
			{
				break;
			}
		}
		return Result;
	}
}


TArray<UWidgetStyleData*> UClassBasedWidgetStyleDefinitions::GetStylesForModifier_Implementation(UVCamModifier* Modifier) const
{
	return UE::VCamExtensions::Private::AccumulateMetaDataRecursively(
		Config,
		Modifier,
		[](const FPerModifierClassWidgetSytleData& ClassConfig)
		{
			return &ClassConfig.ModifierStyles;
		});
}

TArray<UWidgetStyleData*> UClassBasedWidgetStyleDefinitions::GetStylesForConnectionPoint_Implementation(UVCamModifier* Modifier, FName ConnectionPointId) const
{
	return UE::VCamExtensions::Private::AccumulateMetaDataRecursively(
		Config,
		Modifier,
		[ConnectionPointId](const FPerModifierClassWidgetSytleData& ClassConfig)
		{
			return ClassConfig.ConnectionPointStyles.Find(ConnectionPointId);
		});
}

TArray<UWidgetStyleData*> UClassBasedWidgetStyleDefinitions::GetStylesForName_Implementation(FName Category) const
{
	const FWidgetStyleDataArray* Result = CategoriesWithoutModifier.Find(Category);
	return Result
		? Result->Styles
		: TArray<UWidgetStyleData*>{};
}
