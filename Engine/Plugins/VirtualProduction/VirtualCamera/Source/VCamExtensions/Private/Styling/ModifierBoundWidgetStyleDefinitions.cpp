// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ModifierBoundWidgetStyleDefinitions.h"

#include "Styling/WidgetStyleData.h"

namespace UE::VCamExtensions::Private
{
	static UWidgetStyleData* Filter(const TArray<UWidgetStyleData*>& MetaDataArray, TSubclassOf<UWidgetStyleData> Class)
	{
		for (UWidgetStyleData* MetaData : MetaDataArray)
		{
			if (MetaData && MetaData->IsA(Class))
			{
				return MetaData;
			}
		}
		return nullptr;
	}
}

TArray<UWidgetStyleData*> UModifierBoundWidgetStyleDefinitions::GetStylesForModifier_Implementation(UVCamModifier* Modifier) const
{
	unimplemented();
	return {};
}

TArray<UWidgetStyleData*> UModifierBoundWidgetStyleDefinitions::GetStylesForConnectionPoint_Implementation(UVCamModifier* Modifier, FName ConnectionPoint) const
{
	unimplemented();
	return {};
}

TArray<UWidgetStyleData*> UModifierBoundWidgetStyleDefinitions::GetStylesForName_Implementation(FName Category) const
{
	unimplemented();
	return {};
}

UWidgetStyleData* UModifierBoundWidgetStyleDefinitions::GetStyleForModifierByClass(UVCamModifier* Modifier, TSubclassOf<UWidgetStyleData> Class) const
{
	return UE::VCamExtensions::Private::Filter(
		GetStylesForModifier(Modifier),
		Class
		);
}

UWidgetStyleData* UModifierBoundWidgetStyleDefinitions::GetStyleForConnectionPointByClass(UVCamModifier* Modifier, FName ConnectionPoint, TSubclassOf<UWidgetStyleData> Class) const
{
	return UE::VCamExtensions::Private::Filter(
		GetStylesForConnectionPoint(Modifier, ConnectionPoint),
		Class
		);
}

UWidgetStyleData* UModifierBoundWidgetStyleDefinitions::GetStyleForNameByClass(FName Category, TSubclassOf<UWidgetStyleData> Class) const
{
	return UE::VCamExtensions::Private::Filter(
		GetStylesForName(Category),
		Class
		);
}