// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ModifierBoundWidgetStylesAsset.h"

#include "Styling/ClassBasedWidgetStyleDefinitions.h"
#include "Styling/ModifierBoundWidgetStyleDefinitions.h"

TArray<UWidgetStyleData*> UModifierBoundWidgetStylesAsset::GetStylesForModifier(UVCamModifier* Modifier) const
{
	return Rules ? Rules->GetStylesForModifier(Modifier) : TArray<UWidgetStyleData*>{};
}

TArray<UWidgetStyleData*> UModifierBoundWidgetStylesAsset::GetStylesForConnectionPoint(UVCamModifier* Modifier, FName ConnectionPoint) const
{
	return Rules ? Rules->GetStylesForConnectionPoint(Modifier, ConnectionPoint) : TArray<UWidgetStyleData*>{};
}

TArray<UWidgetStyleData*> UModifierBoundWidgetStylesAsset::GetStylesForName(FName Category) const
{
	return Rules ? Rules->GetStylesForName(Category) : TArray<UWidgetStyleData*>{};
}

UWidgetStyleData* UModifierBoundWidgetStylesAsset::GetStyleForModifierByClass(UVCamModifier* Modifier, TSubclassOf<UWidgetStyleData> Class) const
{
	return Rules ? Rules->GetStyleForModifierByClass(Modifier, Class) : nullptr;
}

UWidgetStyleData* UModifierBoundWidgetStylesAsset::GetStyleForConnectionPointByClass(UVCamModifier* Modifier, FName ConnectionPoint, TSubclassOf<UWidgetStyleData> Class) const
{
	return Rules ? Rules->GetStyleForConnectionPointByClass(Modifier, ConnectionPoint, Class) : nullptr;
}

UWidgetStyleData* UModifierBoundWidgetStylesAsset::GetStyleForNameByClass(FName Name, TSubclassOf<UWidgetStyleData> Class) const
{
	return Rules ? Rules->GetStyleForNameByClass(Name, Class) : nullptr;
}
