// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/TargetBasedWidgetStyleDefinitions.h"

#include "Modifier/VCamModifier.h"

TArray<UWidgetStyleData*> UTargetBasedWidgetStyleDefinitions::GetStylesForModifier_Implementation(UVCamModifier* Modifier) const
{
	if (!Modifier)
	{
		return {};
	}
	
	if (const FTargettedModifierStyleConfig* Data = ModifierToStyle.Find(Modifier->GetStackEntryName()))
	{
		TArray<UWidgetStyleData*> Result;
		Algo::Transform(Data->ModifierStyles.Styles, Result, [](const TObjectPtr<UWidgetStyleData>& Data){ return Data; });
		return Result;
	}
	return {};
}

TArray<UWidgetStyleData*> UTargetBasedWidgetStyleDefinitions::GetStylesForConnectionPoint_Implementation(UVCamModifier* Modifier, FName ConnectionPointId) const
{
	if (!Modifier)
	{
		return {};
	}
	
	const FTargettedModifierStyleConfig* Data = ModifierToStyle.Find(Modifier->GetStackEntryName());
	if (!Data)
	{
		return {};
	}

	if (const FWidgetStyleDataArray* Styles = Data->ConnectionPointStyles.Find(ConnectionPointId))
	{
		TArray<UWidgetStyleData*> Result;
		Algo::Transform(Styles->Styles, Result, [](const TObjectPtr<UWidgetStyleData>& Data){ return Data; });
		return Result;
	}
	
	return {};
}

TArray<UWidgetStyleData*> UTargetBasedWidgetStyleDefinitions::GetStylesForName_Implementation(FName Category) const
{
	if (const FWidgetStyleDataArray* Data = CategoriesWithoutModifier.Find(Category))
	{
		TArray<UWidgetStyleData*> Result;
		Algo::Transform(Data->Styles, Result, [](const TObjectPtr<UWidgetStyleData>& Data){ return Data; });
		return Result;
	}
	return {};
}