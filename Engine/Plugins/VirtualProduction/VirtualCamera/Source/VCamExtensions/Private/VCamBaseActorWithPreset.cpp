// Copyright Epic Games, Inc. All Rights Reserved.

#include "VCamBaseActorWithPreset.h"

#include "VCamComponent.h"
#include "BuiltinProviders/VCamPixelStreamingSession.h"
#include "Interfaces/IPluginManager.h"
#include "Output/VCamOutputRemoteSession.h"

namespace UE::VCamExtensions::Private
{
	static bool IsPixelStreamingEnabledPlatform()
	{
		// Instead of #if directives look up the .uplugin file once so this code automatically updates when support changes
		static bool bCache = []()
		{
			IPluginManager& PluginManager = IPluginManager::Get();
			if (const TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(TEXT("VirtualCameraCore")))
			{
				const FModuleDescriptor* PixelStreamingModule = Plugin->GetDescriptor().Modules.FindByPredicate([](const FModuleDescriptor& Module)
				{
					return Module.Name.IsEqual(TEXT("PixelStreamingVCam"));
				});
				return PixelStreamingModule && PixelStreamingModule->IsLoadedInCurrentConfiguration();
			}
			return false;
		}();
		return bCache;
	}
}

void AVCamBaseActorWithPreset::PostActorCreated()
{
	Super::PostActorCreated();

	// Do not do this for actors drag and dropped into the level editor
	if (HasAnyFlags(RF_Transient))
	{
		return;
	}

	// We expect the child Blueprint to create the output providers
	TArray<UVCamOutputProviderBase*> PixelStreamingSessions;
	TArray<UVCamOutputProviderBase*> RemoteSessions;
	GetVCamComponent()->GetOutputProvidersByClass(UVCamPixelStreamingSession::StaticClass(), PixelStreamingSessions);
	GetVCamComponent()->GetOutputProvidersByClass(UVCamOutputRemoteSession::StaticClass(), RemoteSessions);
	
	const bool bEnablePixelStreaming = UE::VCamExtensions::Private::IsPixelStreamingEnabledPlatform();
	for (UVCamOutputProviderBase* PixelStreamer : PixelStreamingSessions)
	{
		PixelStreamer->SetActive(bEnablePixelStreaming);
	}
	for (UVCamOutputProviderBase* RemoteSession : RemoteSessions)
	{
		RemoteSession->SetActive(!bEnablePixelStreaming);
	}
}
