// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::VCamExtensions::Private
{
	class FVCamExtensionsModule : public IModuleInterface
	{
	public:

		virtual void StartupModule() override
		{}
	
		virtual void ShutdownModule() override
		{}
	};
}

IMPLEMENT_MODULE(UE::VCamExtensions::Private::FVCamExtensionsModule, VCamExtensions);