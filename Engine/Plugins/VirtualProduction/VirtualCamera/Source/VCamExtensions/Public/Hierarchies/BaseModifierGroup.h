// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "BaseModifierGroup.generated.h"

/** Implements base logic to keep group names unique within a hierachy */
UCLASS(Abstract)
class VCAMEXTENSIONS_API UBaseModifierGroup : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Hierarchy")
	FName NodeName;

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	//~ End UObject Interface
};
