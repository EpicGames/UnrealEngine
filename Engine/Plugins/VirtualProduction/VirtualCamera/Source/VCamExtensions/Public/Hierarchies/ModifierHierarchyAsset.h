// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "ModifierHierarchyAsset.generated.h"

class UModifierHierarchyRules;
class UVCamComponent;
class UVCamModifier;

struct FVCamModifierConnectionBinding;

/**
 * An asset intended to be referenced by Slate widgets.
 *
 * Defines a tree hierarchy. Nodes are called groups.
 * A group consists of modifiers and (sub) groups.
 * 
 * An example use case is if you want to have a button menu which should procedurally generate sub-button menus depending
 * on the modifiers in the component. 
 * One group could be a Lens group which groups together modifiers that e.g. modify focal distance, filmback, and FOV.
 * The rules are defined generically enough so as modifiers are added or removed, the groupings also update accordingly.
 */
UCLASS(BlueprintType)
class VCAMEXTENSIONS_API UModifierHierarchyAsset : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = "Virtual Camera")
	TObjectPtr<UModifierHierarchyRules> Rules;

	/** Gets the root of the tree. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	FName GetRootNode() const;

	/** Gets the parent of this given group. Fails if called on the root node. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	bool GetParentNode(FName ChildNode, FName& ParentNode) const;
	
	/** Gets the child groups of the given group. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	TSet<FName> GetChildNodes(FName ParentGroup) const;

	/** Gets all the modifiers on the component that belong in the given group. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	UVCamModifier* GetModifierInNode(UVCamComponent* Component, FName GroupName) const;
	
	/**
	 * Gets the connection point the modifier is configured to be bound to.
	 * This function is optional to implement; it is valid for it to always return false.
	 */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	bool GetConnectionPointTargetForNode(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connection) const;
	
	/** Utility function to get all groups which contain this modifier. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Hierarchies")
	TSet<FName> GetGroupsContainingModifier(UVCamModifier* Modifier) const;
};