// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/VCamConnectionStructs.h"
#include "UObject/Object.h"
#include "ModifierHierarchyRules.generated.h"

class UVCamComponent;
class UVCamModifier;

USTRUCT(BlueprintType)
struct FVCamModifierConnectionBinding
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "Virtual Camera")
	TObjectPtr<UVCamModifier> Modifier;

	UPROPERTY(BlueprintReadWrite, Category = "Virtual Camera")
	FName ConnectionPoint;
};

/**
 * Defines a tree hierarchy. Each node is called a group.
 * A group consists of connections and (sub) groups. A connection is a modifier and a corresponding connection point.
 * 
 * An example use case is if you want to have a button menu which should procedurally generate sub-button menus depending
 * on the modifiers in the component. 
 * One group could be a Lens group which groups together modifiers that e.g. modify focal distance, filmback, and FOV.
 * The rules are defined generically enough so as modifiers are added or removed, the groupings also update accordingly.
 */
UCLASS(Abstract, Blueprintable, EditInlineNew)
class VCAMEXTENSIONS_API UModifierHierarchyRules : public UObject
{
	GENERATED_BODY()
public:

	/** Gets the root of the tree. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	FName GetRootNode() const;

	/** Gets the parent of this given group. Fails if called on the root node. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	bool GetParentNode(FName ChildNode, FName& ParentNode) const;
	
	/** Gets the child groups of the given group. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	TSet<FName> GetChildNodes(FName Node) const;

	/** Gets all the modifiers on the component that belong in the given group. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	UVCamModifier* GetModifierInNode(UVCamComponent* Component, FName NodeName) const;
	
	/**
	 * Gets the connection point the modifier is configured to be bound to, if any.
	 * This function is optional to implement; it is valid for it to always return false.
	 * @return Whether there is a connection point configured for this hierarchy node.
	 */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	bool GetConnectionPointTargetForNode(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connection) const;
	
	/** Utility function to get all groups which contain this modifier. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera|Hierarchies")
	TSet<FName> GetNodesContainingModifier(UVCamModifier* Modifier) const;
};
