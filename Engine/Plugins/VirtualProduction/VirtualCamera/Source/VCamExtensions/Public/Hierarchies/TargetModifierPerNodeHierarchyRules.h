// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseModifierGroup.h"
#include "ModifierHierarchyRules.h"
#include "Templates/Function.h"
#include "UI/VCamConnectionStructs.h"
#include "TargetModifierPerNodeHierarchyRules.generated.h"

UCLASS(EditInlineNew)
class VCAMEXTENSIONS_API USingleModifierPerNodeWithTargetSettings : public UBaseModifierGroup
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Hierarchy")
	FVCamConnectionTargetSettings TargetSettings;

	UPROPERTY(EditAnywhere, Instanced, Category = "Hierarchy")
	TArray<TObjectPtr<USingleModifierPerNodeWithTargetSettings>> ChildElements;
};

/**
 * 
 */
UCLASS()
class VCAMEXTENSIONS_API UTargetModifierPerNodeHierarchyRules : public UModifierHierarchyRules
{
	GENERATED_BODY()
public:
	
	UTargetModifierPerNodeHierarchyRules();

	//~ Begin UModifierHierarchyRules Interface
	virtual FName GetRootNode_Implementation() const override;
	virtual bool GetParentNode_Implementation(FName ChildGroup, FName& ParentGroup) const override;
	virtual TSet<FName> GetChildNodes_Implementation(FName ParentGroup) const override;
	virtual UVCamModifier* GetModifierInNode_Implementation(UVCamComponent* Component, FName GroupName) const override;
	virtual bool GetConnectionPointTargetForNode_Implementation(FName GroupName, UVCamComponent* Component, FVCamModifierConnectionBinding& Connection) const override;
	virtual TSet<FName> GetNodesContainingModifier_Implementation(UVCamModifier* Modifier) const override;
	//~ End UModifierHierarchyRules Interface

private:

	UPROPERTY(EditAnywhere, Instanced, Category = "Hierarchy")
	TObjectPtr<USingleModifierPerNodeWithTargetSettings> Root;

	TSet<USingleModifierPerNodeWithTargetSettings*> GetNodeForModifier(UVCamModifier* Modifier) const;
	USingleModifierPerNodeWithTargetSettings* FindNodeByName(FName GroupName) const;
	
	enum class EBreakBehavior
	{
		Continue,
		Break
	};
	void ForEachGroup(TFunctionRef<EBreakBehavior(USingleModifierPerNodeWithTargetSettings& CurrentGroup, USingleModifierPerNodeWithTargetSettings* Parent)> Callback) const;
};
