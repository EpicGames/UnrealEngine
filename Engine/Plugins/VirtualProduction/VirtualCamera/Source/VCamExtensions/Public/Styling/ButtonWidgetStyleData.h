// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetStyleData.h"
#include "Styling/SlateTypes.h"
#include "ButtonWidgetStyleData.generated.h"

/** Associates a button style for widgets that want to display a button for a modifier / connection point. */
UCLASS()
class VCAMEXTENSIONS_API UButtonWidgetStyleData : public UWidgetStyleData
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Virtual Camera")
	FButtonStyle ButtonStyle;
};