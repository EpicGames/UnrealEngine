// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModifierBoundWidgetStyleDefinitions.h"
#include "Modifier/VCamModifier.h"
#include "WidgetStyleDataArray.h"
#include "ClassBasedWidgetStyleDefinitions.generated.h"

class UWidgetStyleData;

USTRUCT()
struct VCAMEXTENSIONS_API FWidgetStyleDataConfig
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Instanced, Category = "Virtual Camera")
	TArray<TObjectPtr<UWidgetStyleData>> ModifierMetaData;
	
	/** Whether to inherit the meta data of modifier's parent class(es). */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera", AdvancedDisplay)
	bool bInherit = false;
};

USTRUCT()
struct VCAMEXTENSIONS_API FPerModifierClassWidgetSytleData
{
	GENERATED_BODY()

	/** Data to attach to modifiers matching the associated class. */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	FWidgetStyleDataConfig ModifierStyles;

	/** Meta data that is attached to a custom name, such as a connection point or group name (if using e.g. UClassBasedModifierHierarchyRules) */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<FName, FWidgetStyleDataConfig> ConnectionPointStyles;
};

/**
 * Assigns meta data based on modifier class. To every modifier class you can assign meta data.
 */
UCLASS()
class VCAMEXTENSIONS_API UClassBasedWidgetStyleDefinitions : public UModifierBoundWidgetStyleDefinitions
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<TSubclassOf<UVCamModifier>, FPerModifierClassWidgetSytleData> Config;

	/** Meta data that is not bound to any modifier but just to a simple name. */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<FName, FWidgetStyleDataArray> CategoriesWithoutModifier;
	
	//~ Begin UModifierMetaDataRules Interface
	virtual TArray<UWidgetStyleData*> GetStylesForModifier_Implementation(UVCamModifier* Modifier) const override;
	virtual TArray<UWidgetStyleData*> GetStylesForConnectionPoint_Implementation(UVCamModifier* Modifier, FName ConnectionPointId) const override;
	virtual TArray<UWidgetStyleData*> GetStylesForName_Implementation(FName Category) const override;
	//~ End UModifierMetaDataRules Interface
};
