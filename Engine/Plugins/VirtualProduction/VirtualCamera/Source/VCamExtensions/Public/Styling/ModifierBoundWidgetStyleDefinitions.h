// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetStyleData.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "ModifierBoundWidgetStyleDefinitions.generated.h"

class UWidgetStyleData;
class UVCamModifier;

/**
 * Associates information with modifiers and their connection points.
 * You can use it to retrieve custom information assigned to a modifier and / or its connections,
 * such as what icon a button representing that widget should have.
 */
UCLASS(Abstract, Blueprintable, EditInlineNew)
class VCAMEXTENSIONS_API UModifierBoundWidgetStyleDefinitions : public UObject
{
	GENERATED_BODY()
public:

	/** Retrieves all meta data that is associated for a given modifier. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForModifier(UVCamModifier* Modifier) const;

	/** Retrieves all meta data that is associated for a given modifier and a sub-category name. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForConnectionPoint(UVCamModifier* Modifier, FName ConnectionPoint) const;

	/** Retrieves all meta data that is associated with a given a category name; this data is not associated with any kind of modifier. */
	UFUNCTION(BlueprintPure, BlueprintNativeEvent, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForName(FName Category) const;

	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForModifierByClass(UVCamModifier* Modifier, TSubclassOf<UWidgetStyleData> Class) const;
	
	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForConnectionPointByClass(UVCamModifier* Modifier, FName ConnectionPoint, TSubclassOf<UWidgetStyleData> Class) const;
	
	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForNameByClass(FName Category, TSubclassOf<UWidgetStyleData> Class) const;
};
