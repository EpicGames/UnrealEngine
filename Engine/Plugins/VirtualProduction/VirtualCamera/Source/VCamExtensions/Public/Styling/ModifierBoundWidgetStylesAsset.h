// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "ModifierBoundWidgetStyleDefinitions.h"
#include "ModifierBoundWidgetStylesAsset.generated.h"

/**
 * An asset intended to be referenced by Slate widgets.
 * 
 * For example, you can retrieve custom style info assigned to a modifier and / or its connections,
 * such as what icon a button representing that widget should have.
 */
UCLASS(BlueprintType)
class VCAMEXTENSIONS_API UModifierBoundWidgetStylesAsset : public UObject
{
	GENERATED_BODY()
public:
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Instanced, Category = "Virtual Camera")
	TObjectPtr<UModifierBoundWidgetStyleDefinitions> Rules;

	/** Retrieves all meta data that is associated for a given modifier. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForModifier(UVCamModifier* Modifier) const;

	/** Retrieves all meta data that is associated for a given modifier and a sub-category name. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForConnectionPoint(UVCamModifier* Modifier, FName ConnectionPoint) const;

	/** Retrieves all meta data that is associated with a given a category name; this data is not associated with any kind of modifier. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera")
	TArray<UWidgetStyleData*> GetStylesForName(FName Category) const;

	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForModifierByClass(UVCamModifier* Modifier, TSubclassOf<UWidgetStyleData> Class) const;
	
	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForConnectionPointByClass(UVCamModifier* Modifier, FName ConnectionPoint, TSubclassOf<UWidgetStyleData> Class) const;
	
	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (DeterminesOutputType = "Class"))
	UWidgetStyleData* GetStyleForNameByClass(FName Name, TSubclassOf<UWidgetStyleData> Class) const;
};