// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModifierBoundWidgetStyleDefinitions.h"
#include "WidgetStyleDataArray.h"
#include "TargetBasedWidgetStyleDefinitions.generated.h"

USTRUCT()
struct VCAMEXTENSIONS_API FTargettedModifierStyleConfig
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	FWidgetStyleDataArray ModifierStyles;

	/** Key: Name of a connection point on the modifier */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<FName, FWidgetStyleDataArray> ConnectionPointStyles;
};

UCLASS()
class VCAMEXTENSIONS_API UTargetBasedWidgetStyleDefinitions : public UModifierBoundWidgetStyleDefinitions
{
	GENERATED_BODY()
public:

	/** Key: Name of a modifier in a VCam's stack entry. */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<FName, FTargettedModifierStyleConfig> ModifierToStyle;

	/** Styles that is not bound to any modifier but just to a name. */
	UPROPERTY(EditAnywhere, Category = "Virtual Camera")
	TMap<FName, FWidgetStyleDataArray> CategoriesWithoutModifier;

	//~ Begin UModifierMetaDataRules Interface
	virtual TArray<UWidgetStyleData*> GetStylesForModifier_Implementation(UVCamModifier* Modifier) const override;
	virtual TArray<UWidgetStyleData*> GetStylesForConnectionPoint_Implementation(UVCamModifier* Modifier, FName ConnectionPointId) const override;
	virtual TArray<UWidgetStyleData*> GetStylesForName_Implementation(FName Category) const override;
	//~ End UModifierMetaDataRules Interface
};
