// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "WidgetStyleData.generated.h"

/** Base class for data that can be associated with modifiers / their connection points. */
UCLASS(Abstract, Blueprintable, EditInlineNew)
class VCAMEXTENSIONS_API UWidgetStyleData : public UObject
{
	GENERATED_BODY()
};