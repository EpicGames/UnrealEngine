// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetStyleDataArray.generated.h"

class UWidgetStyleData;

USTRUCT()
struct VCAMEXTENSIONS_API FWidgetStyleDataArray
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Instanced, Category = "Virtual Camera")
	TArray<TObjectPtr<UWidgetStyleData>> Styles;
};