// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VCamBaseActor.h"
#include "VCamBaseActorWithPreset.generated.h"

/**
 * Base class for the VCamActor preset Blueprint.
 * 
 * If the platform supports pixel streaming, this activates all pixel streaming and deactivates all remote session output providers.
 * The child Blueprint is expected to contain 1 pixel streaming and 1 remote session output provider.
 */
UCLASS(Abstract)
class VCAMEXTENSIONS_API AVCamBaseActorWithPreset : public AVCamBaseActor
{
	GENERATED_BODY()
public:

	//~ Begin AActor Interface
	virtual void PostActorCreated() override;
	//~ End AActor Interface
};
