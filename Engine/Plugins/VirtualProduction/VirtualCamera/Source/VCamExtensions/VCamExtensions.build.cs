// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VCamExtensions : ModuleRules
{
	public VCamExtensions(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core", 
				"VCamCore",
				"CinematicCamera"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DecoupledOutputProvider",
				"Engine",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore", 
			}
		);
	}
}
