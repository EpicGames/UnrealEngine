// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ModifierBoundWidgetStyles.h"

#include "Styling/ModifierBoundWidgetStylesAsset.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_ModifierBoundWidgetStyles"

FText UAssetDefinition_ModifierBoundWidgetStyles::GetAssetDisplayName() const
{
	return LOCTEXT("ModifierBoundWidgetStyles_AssetName", "Modifier Bound Widget Styles");
}

TSoftClassPtr<UObject> UAssetDefinition_ModifierBoundWidgetStyles::GetAssetClass() const
{
	return UModifierBoundWidgetStylesAsset::StaticClass();
}

FLinearColor UAssetDefinition_ModifierBoundWidgetStyles::GetAssetColor() const
{
	return FColor(201, 29, 85);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ModifierBoundWidgetStyles::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("ModifierBoundWidgetStyles_AssetCategory", "Virtual Production")), LOCTEXT("ModifierBoundWidgetStyles_CategorySection", "Virtual Camera"), ECategoryMenuType::Section)
	};

	return Categories;
}

#undef LOCTEXT_NAMESPACE
