// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ModifierHierarchy.h"

#include "Hierarchies/ModifierHierarchyAsset.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_ModifierHierarchy"

FText UAssetDefinition_ModifierHierarchy::GetAssetDisplayName() const
{
	return LOCTEXT("ModifierHierarchy_AssetName", "Modifier Hierarchy");
}

TSoftClassPtr<UObject> UAssetDefinition_ModifierHierarchy::GetAssetClass() const
{
	return UModifierHierarchyAsset::StaticClass();
}

FLinearColor UAssetDefinition_ModifierHierarchy::GetAssetColor() const
{
	return FColor(201, 29, 85);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ModifierHierarchy::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("ModifierHierarchy_AssetCategory", "Virtual Production")), LOCTEXT("ModifierHierarchy_CategorySection", "Virtual Camera"), ECategoryMenuType::Section)
	};

	return Categories;
}

#undef LOCTEXT_NAMESPACE
