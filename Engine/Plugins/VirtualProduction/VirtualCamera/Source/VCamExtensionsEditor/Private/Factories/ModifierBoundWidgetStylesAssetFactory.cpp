// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifierBoundWidgetStylesAssetFactory.h"

#include "IVCamCoreEditorModule.h"
#include "Styling/ModifierBoundWidgetStylesAsset.h"

#define LOCTEXT_NAMESPACE "UModifierMetaDataAssetFactory"

UModifierBoundWidgetStylesAssetFactory::UModifierBoundWidgetStylesAssetFactory()
{
	SupportedClass = UModifierBoundWidgetStylesAsset::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

FText UModifierBoundWidgetStylesAssetFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Modifier Bound Widget Styles");
}

FText UModifierBoundWidgetStylesAssetFactory::GetToolTip() const
{
	return UModifierBoundWidgetStylesAsset::StaticClass()->GetToolTipText();
}

UObject* UModifierBoundWidgetStylesAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UModifierBoundWidgetStylesAsset>(InParent, Class, Name, Flags);
}

uint32 UModifierBoundWidgetStylesAssetFactory::GetMenuCategories() const
{
	return UE::VCamCoreEditor::IVCamCoreEditorModule::Get().GetAdvancedAssetCategoryForVCam();
}

const TArray<FText>& UModifierBoundWidgetStylesAssetFactory::GetMenuCategorySubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("UModifierMetaDataAssetFactory_SubMenu", "Extensions")
	};

	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
