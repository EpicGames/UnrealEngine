// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "ModifierBoundWidgetStylesAssetFactory.generated.h"

/**
 * 
 */
UCLASS()
class UModifierBoundWidgetStylesAssetFactory : public UFactory
{
	GENERATED_BODY()
public:

	UModifierBoundWidgetStylesAssetFactory();

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	//~ End UFactory Interface
};
