// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModifierHierarchyAssetFactory.h"

#include "IVCamCoreEditorModule.h"
#include "Hierarchies/ModifierHierarchyAsset.h"

#define LOCTEXT_NAMESPACE "UModifierHierarchyAssetFactory"

UModifierHierarchyAssetFactory::UModifierHierarchyAssetFactory()
{
	SupportedClass = UModifierHierarchyAsset::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

FText UModifierHierarchyAssetFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Modifier Hierarchy");
}

FText UModifierHierarchyAssetFactory::GetToolTip() const
{
	return UModifierHierarchyAsset::StaticClass()->GetToolTipText();
}

UObject* UModifierHierarchyAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UModifierHierarchyAsset>(InParent, Class, Name, Flags);
}

uint32 UModifierHierarchyAssetFactory::GetMenuCategories() const
{
	return UE::VCamCoreEditor::IVCamCoreEditorModule::Get().GetAdvancedAssetCategoryForVCam();
}

const TArray<FText>& UModifierHierarchyAssetFactory::GetMenuCategorySubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("UModifierMetaDataAssetFactory_SubMenu", "Extensions")
	};

	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
