// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "ModifierHierarchyAssetFactory.generated.h"

/**
 * 
 */
UCLASS()
class UModifierHierarchyAssetFactory : public UFactory
{
	GENERATED_BODY()
public:

	UModifierHierarchyAssetFactory();

	//~ Begin UFactory Interface
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	//~ End UFactory Interface
};
