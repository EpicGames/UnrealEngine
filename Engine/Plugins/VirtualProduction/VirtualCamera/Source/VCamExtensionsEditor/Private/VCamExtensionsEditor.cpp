// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace UE::VCamExtensionsEditor::Private
{
	class FVCamExtensionsEditorModule : public IModuleInterface
	{
	public:
		
		virtual void StartupModule() override
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", NSLOCTEXT("VCamExtensionsEditor", "VirtualProductionCategory", "Virtual Production"));
		}
		
		virtual void ShutdownModule() override
		{
		}
	};
}

IMPLEMENT_MODULE(UE::VCamExtensionsEditor::Private::FVCamExtensionsEditorModule, VCamExtensionsEditor);