// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VCamExtensionsEditor : ModuleRules
{
	public VCamExtensionsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"AssetTools",
				"CoreUObject",
				"Engine",
				"UnrealEd",
				"VCamCore",
				"VCamCoreEditor",
				"VCamExtensions"
			}
		);
	}
}
