// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSpawnableVCamBinding.h"

#include "CineCameraActor.h"
#include "Cinematic/VCamHierarchyInfo.h"
#include "Engine/Engine.h"
#include "MovieScene.h"

bool UMovieSceneSpawnableVCamBinding::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	return SourceObject && UE::VirtualCamera::FVCamHierarchyInfo(SourceObject).ShouldRecordAsCineCamera();
}

UMovieSceneCustomBinding* UMovieSceneSpawnableVCamBinding::CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	const UE::VirtualCamera::FVCamHierarchyInfo VCamInfo(SourceObject);
	if (!SourceObject || !VCamInfo.ShouldRecordAsCineCamera()) // We don't expect this to happen as SupportsBindingCreationFromObject rejects this case.
	{
		return nullptr;
	}
	
	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : TEXT("EmptyBinding"));
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	ACineCameraActor* CineCameraActor = NewObject<ACineCameraActor>(&OwnerMovieScene, TemplateName, RF_Transactional);
	if (!CineCameraActor)
	{
		return nullptr;
	}
	
	UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
	CopyParams.bNotifyObjectReplacement = false;
	CopyParams.bPreserveRootComponent = false;
	CopyParams.bPerformDuplication = true;
	CopyParams.bDoDelta = false; // Required so all properties are copied over correctly. 
	UEngine::CopyPropertiesForUnrelatedObjects(SourceObject, CineCameraActor, CopyParams);

	// Specific for VCam: The asymmetric overscan needs to be zero in the resulting level sequence.
	// The SensorCorrectionModifier sets the asymmetric overscan to be non-zero to achieve two "viewports" in the VCam UI.
	//  - In inner, uncropped viewport which is supposed to be what the CineCameraActor is supposed to see after recording is done.
	//  - The outer, cropped viewport, which is darker in the VCam UI. That section would be visible if we didn't reset asymmetric overscan here.
	CineCameraActor->GetCineCameraComponent()->AsymmetricOverscan = FVector4f::Zero();
	CineCameraActor->GetCineCameraComponent()->bConstrainAspectRatio = true;
	
	CineCameraActor->Tags.Remove(TEXT("SequencerActor"));
	CineCameraActor->Tags.Remove(TEXT("SequencerPreviewActor"));
	CineCameraActor->DetachFromActor(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
#if WITH_EDITOR
	CineCameraActor->bIsEditorPreviewActor = false;
#endif

	// Record as if this was CineCameraActor.
	// Do not create UMovieSceneSpawnableVCamBinding instance because that class won't be available if the user disables the Virtual Camera plugin.
	UMovieSceneSpawnableActorBinding* Binding = NewObject<UMovieSceneSpawnableActorBinding>(
		&OwnerMovieScene, UMovieSceneSpawnableActorBinding::StaticClass(), InstancedBindingName, RF_Transactional
		);
	Binding->SetObjectTemplate(CineCameraActor);
	return Binding;
}