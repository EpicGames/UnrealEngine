// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "MovieSceneSpawnableVCamBinding.generated.h"

namespace UE::TakeRecorderSources{ struct FCanRecordArgs; }

/**
 * Allows the AVCamBaseActor to be recorded as ACineCameraActor instead.
 *
 * This allows users to record a VCam actor using Take Recorder and disable the VCam plugin afterwards while still being able to use the recorded
 * sequences without missing classes.
 *
 * This class is never actually instantiated. It simply creates a UMovieSceneSpawnableActorBinding that contains a ACineCameraActor instead of a VCam.
 * Not instantiating this class is important: otherwise the sequence referencing the binding would lose the reference when VCam plugin is disabled as
 * the class would be missing.
 */
UCLASS()
class UMovieSceneSpawnableVCamBinding : public UMovieSceneSpawnableActorBinding
{
	GENERATED_BODY()
public:

	//~ Begin UMovieSceneSpawnableActorBinding Interface
	virtual bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override;
	virtual UMovieSceneCustomBinding* CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override;
	//~ End UMovieSceneSpawnableActorBinding Interface

protected:
	
	//~ Begin UMovieSceneSpawnableActorBinding Interface
	virtual int32 GetCustomBindingPriority() const override { return BaseCustomPriority; }
	//~ End UMovieSceneSpawnableActorBinding Interface
};