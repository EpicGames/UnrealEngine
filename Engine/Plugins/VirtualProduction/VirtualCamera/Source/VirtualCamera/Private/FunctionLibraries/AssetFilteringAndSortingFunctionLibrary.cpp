// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraries/AssetFilteringAndSortingFunctionLibrary.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"

#include "Logging/LogMacros.h"

#include "Misc/ComparisonUtility.h"

DEFINE_LOG_CATEGORY_STATIC(LogAssetFilteringAndSorting, All, All);

namespace UE::VirtualCamera::Private
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	template<typename TType>
	static bool Sort(TArray<FAssetData>& Assets, FName MetaDataTag, TFunctionRef<bool(const FString& TagType, TType& Converted)> Converter, ESortOrder SortOrder)
	{
		TMap<FSoftObjectPath, TType> MetaData;
		MetaData.Reserve(Assets.Num());
	
		for (const FAssetData& AssetData : Assets)
		{
			const FAssetTagValueRef Value = AssetData.TagsAndValues.FindTag(MetaDataTag);
			TType AssetTagValue;
			if (Value.IsSet() && Converter(Value.GetValue(), AssetTagValue))
			{
				MetaData.Add(AssetData.GetSoftObjectPath(), AssetTagValue);
			}
			else
			{
				UE_LOGF(LogAssetFilteringAndSorting, Warning, "Not all assets have the tag '%ls'", *MetaDataTag.ToString());
				return false;
			}
		}

		UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::SortAssets(Assets, [&MetaData](const FAssetData& Left, const FAssetData& Right)
		{
			return MetaData[Left.GetSoftObjectPath()] <= MetaData[Right.GetSoftObjectPath()]; 
		}, SortOrder);
		return true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TArray<FAssetData> UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::GetAllAssetsByMetaDataTags(const TSet<FName>& RequiredTags, const TSet<UClass*>& AllowedClasses)
{
	FARFilter Filter;
	
	Filter.TagsAndValues.Reserve(RequiredTags.Num());
	for (FName RequiredTag : RequiredTags)
	{
		Filter.TagsAndValues.Add(RequiredTag);
	}

	Filter.ClassPaths.Reserve(RequiredTags.Num());
	for (UClass* AllowedClass : AllowedClasses)
	{
		Filter.ClassPaths.Add(FTopLevelAssetPath(AllowedClass));
	}

	TArray<FAssetData> Result;
	IAssetRegistry::Get()->GetAssets(Filter, Result);
	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::SortByCustomPredicate(TArray<FAssetData>& Assets, FAssetSortingPredicate SortingPredicate, ESortOrder SortOrder)
{
	if (SortingPredicate.IsBound())
	{
		SortAssets(
			Assets,
			[&SortingPredicate](const FAssetData& Left, const FAssetData& Right)
			{
				return SortingPredicate.Execute(Left, Right);
			}, SortOrder);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::SortByAssetName(TArray<FAssetData>& Assets, ESortOrder SortOrder)
{
	SortAssets(Assets,
		[](const FAssetData& Left, const FAssetData& Right)
		{
			return UE::ComparisonUtility::CompareNaturalOrder(Left.AssetName.ToString(), Right.AssetName.ToString()) < 0;
		}, SortOrder);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::SortByMetaData(TArray<FAssetData>& Assets, FName MetaDataTag, EAssetTagMetaDataSortType MetaDataType, ESortOrder SortOrder)
{
	switch (MetaDataType)
	{
	case EAssetTagMetaDataSortType::String:
		return UE::VirtualCamera::Private::Sort<FString>(Assets, MetaDataTag, [](const FString& String, FString& Result){ Result = String; return true; }, SortOrder);
	case EAssetTagMetaDataSortType::Numeric:
		return UE::VirtualCamera::Private::Sort<double>(Assets, MetaDataTag, [](const FString& String, double& Result)
		{
			if (String.IsNumeric())
			{
				Result = FCString::Atod(*String);
				return true;
			}
			return false;
		}, SortOrder);
	case EAssetTagMetaDataSortType::DateTime:
		return UE::VirtualCamera::Private::Sort<FDateTime>(Assets, MetaDataTag, [](const FString& String, FDateTime& Result){ return FDateTime::Parse(String, Result); }, SortOrder);
	default:
		checkNoEntry();
		return false;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDEPRECATED_AssetFilteringAndSortingFunctionLibrary::SortAssets(TArray<FAssetData>& Assets, TFunctionRef<bool(const FAssetData& Left, const FAssetData& Right)> Predicate, ESortOrder SortOrder)
{
	// Careful this would be undefined behaviour: TFunctionRef Variable = [](){}; 
	auto ReversePredicate = [&Predicate](const FAssetData& Left, const FAssetData& Right) { return !Predicate(Left, Right); };
	const TFunctionRef<bool(const FAssetData& Left, const FAssetData& Right)> ReversePredicateFunc = ReversePredicate;
	const TFunctionRef<bool(const FAssetData& Left, const FAssetData& Right)> PredicateToUse = SortOrder == ESortOrder::Ascending ? Predicate : ReversePredicateFunc;
	
	Assets.Sort(PredicateToUse);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
