// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraries/GameViewFunctionLibrary.h"

#include "EVCamTargetViewportID.h"

#if WITH_EDITOR
#include "SLevelViewport.h"
#include "VirtualCamera.h"
#endif

void UGameViewFunctionLibrary::ToggleGameView(EVCamTargetViewportID ViewportID)
{
#if WITH_EDITOR
	if (const TSharedPtr<SLevelViewport> Viewport = UE::VCamCore::GetLevelViewport(ViewportID))
	{
		Viewport->ToggleGameView();
	}
	else
	{
		UE_LOGF(LogVirtualCamera, Warning, "Failed to ToggleGameView for viewport %ls", *UE::VCamCore::ViewportIdToString(ViewportID));
	}
#endif
}

bool UGameViewFunctionLibrary::CanToggleGameView(EVCamTargetViewportID ViewportID)
{
#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TSharedPtr<SLevelViewport> Viewport = UE::VCamCore::GetLevelViewport(ViewportID);
	return Viewport && Viewport->CanToggleGameView();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
	return false;
#endif
}

bool UGameViewFunctionLibrary::IsInGameView(EVCamTargetViewportID ViewportID)
{
#if WITH_EDITOR
	const TSharedPtr<SLevelViewport> Viewport = UE::VCamCore::GetLevelViewport(ViewportID);
	return Viewport && Viewport->IsInGameView();
#else
	return true;
#endif
}

void UGameViewFunctionLibrary::SetGameViewEnabled(EVCamTargetViewportID ViewportID, bool bIsEnabled)
{
	if ((IsInGameView(ViewportID) && !bIsEnabled) || (!IsInGameView(ViewportID) && bIsEnabled))
	{
		ToggleGameView(ViewportID);
	}
}

void UGameViewFunctionLibrary::SetGameViewEnabledForAllViewports(bool bIsEnabled)
{
	for (int32 i = 0; i < static_cast<int32>(EVCamTargetViewportID::Count); ++i)
	{
		SetGameViewEnabled(static_cast<EVCamTargetViewportID>(i), bIsEnabled);
	}
}

TMap<EVCamTargetViewportID, bool> UGameViewFunctionLibrary::SnapshotGameViewStates()
{
	TMap<EVCamTargetViewportID, bool> Result;
#if WITH_EDITOR
	for (int32 i = 0; i < static_cast<int32>(EVCamTargetViewportID::Count); ++i)
	{
		const EVCamTargetViewportID ViewportID = static_cast<EVCamTargetViewportID>(i);
		if (const TSharedPtr<SLevelViewport> Viewport = UE::VCamCore::GetLevelViewport(ViewportID))
		{
			Result.Add(ViewportID, Viewport->IsInGameView());
		}
	}
#endif
	return Result;
}

void UGameViewFunctionLibrary::RestoreGameViewStates(const TMap<EVCamTargetViewportID, bool>& Snapshot)
{
#if WITH_EDITOR
	for (const TPair<EVCamTargetViewportID, bool>& SnapshotPair : Snapshot)
	{
		SetGameViewEnabled(SnapshotPair.Key, SnapshotPair.Value);
	}
#endif
}
