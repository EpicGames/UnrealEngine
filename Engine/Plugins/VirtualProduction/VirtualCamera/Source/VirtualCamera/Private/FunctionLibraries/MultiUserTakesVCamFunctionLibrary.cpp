// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraries/MultiUserTakesVCamFunctionLibrary.h"

#if WITH_EDITOR
#include "MultiUserTakesFunctionLibrary.h"
#endif

bool UMultiUserTakesVCamFunctionLibrary::GetRecordOnClientLocal()
{
#if WITH_EDITOR
	return UMultiUserTakesFunctionLibrary::GetRecordOnClientLocal();
#else
	return false;
#endif
}

void UMultiUserTakesVCamFunctionLibrary::SetRecordOnClientLocal(bool bNewValue)
{
#if WITH_EDITOR
	UMultiUserTakesFunctionLibrary::SetRecordOnClientLocal(bNewValue);
#endif
}

bool UMultiUserTakesVCamFunctionLibrary::GetRecordOnClient(const FGuid& ClientEndpointId)
{
#if WITH_EDITOR
	return UMultiUserTakesFunctionLibrary::GetRecordOnClient(ClientEndpointId);
#else
	return false;
#endif
}

void UMultiUserTakesVCamFunctionLibrary::SetRecordOnClient(const FGuid& ClientEndpointId, bool bNewValue)
{
#if WITH_EDITOR
	UMultiUserTakesFunctionLibrary::SetRecordOnClient(ClientEndpointId, bNewValue);
#endif
}

bool UMultiUserTakesVCamFunctionLibrary::GetSynchronizeTakeRecorderTransactionsLocal()
{
#if WITH_EDITOR
	return UMultiUserTakesFunctionLibrary::GetSynchronizeTakeRecorderTransactionsLocal();
#else
	return false;
#endif
}

bool UMultiUserTakesVCamFunctionLibrary::GetSynchronizeTakeRecorderTransactions(const FGuid& ClientEndpointId)
{
#if WITH_EDITOR
	return UMultiUserTakesFunctionLibrary::GetSynchronizeTakeRecorderTransactions(ClientEndpointId);
#else
	return false;
#endif
}

void UMultiUserTakesVCamFunctionLibrary::SetSynchronizeTakeRecorderTransactionsLocal(bool bNewValue)
{
#if WITH_EDITOR
	UMultiUserTakesFunctionLibrary::SetSynchronizeTakeRecorderTransactionsLocal(bNewValue);
#endif
}
