// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraries/TakeMetaDataTagsFunctionLibrary.h"

#if WITH_EDITOR
#include "TakeMetaData.h"
#endif

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_Slate()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_Slate;
#else
	return "TakeMetaData_Slate";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_TakeNumber()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_TakeNumber;
#else
	return "TakeMetaData_TakeNumber";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_Timestamp()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_Timestamp;
#else
	return "TakeMetaData_Timestamp";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_TimecodeIn()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_TimecodeIn;
#else
	return "TakeMetaData_TimecodeIn";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_TimecodeOut()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_TimecodeOut;
#else
	return "TakeMetaData_TimecodeOut";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_Description()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_Description;
#else
	return "TakeMetaData_Description";
#endif
}

FName UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_LevelPath()
{
#if WITH_EDITOR
	return UTakeMetaData::AssetRegistryTag_LevelPath;
#else
	return "TakeMetaData_LevelPath";
#endif
}

TSet<FName> UTakeMetaDataTagsFunctionLibrary::GetAllTakeMetaDataTags()
{
	return {
		GetTakeMetaDataTag_Slate(),
		GetTakeMetaDataTag_TakeNumber(),
		GetTakeMetaDataTag_Timestamp(),
		GetTakeMetaDataTag_TimecodeIn(),
		GetTakeMetaDataTag_TimecodeOut(),
		GetTakeMetaDataTag_Description(),
		GetTakeMetaDataTag_LevelPath()
	};
}
