// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionLibraries/VCamBlueprintFunctionLibrary.h"

#include "AssetRegistry/AssetData.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "EngineUtils.h"
#include "Engine/GameInstance.h"
#include "FunctionLibraries/TakeMetaDataTagsFunctionLibrary.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "LevelSequence.h"
#include "Misc/ExpressionParserTypes.h"
#include "MovieScene.h"
#include "MovieSceneSequencePlayer.h"
#include "Slate/SceneViewport.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorAssetLibrary.h"
#include "EVCamTargetViewportID.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "ITakeRecorderModule.h"
#include "LevelEditor.h"
#include "LevelEditorSubsystem.h"
#include "LevelEditorViewport.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "Modules/ModuleManager.h"
#include "Recorder/TakeRecorderBlueprintLibrary.h"
#include "Recorder/TakeRecorderPanel.h"
#include "SLevelViewport.h"
#include "SceneView.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TakesCoreBlueprintLibrary.h"
#include "TakePreset.h"
#include "VPUtilitiesEditorBlueprintLibrary.h"
#endif

bool UVCamBlueprintFunctionLibrary::IsGameRunning()
{
#if WITH_EDITOR
	return (GEditor && GEditor->IsPlaySessionInProgress());
#else
	return true;
#endif
}

ULevelSequence* UVCamBlueprintFunctionLibrary::GetCurrentLevelSequence()
{
#if WITH_EDITOR
	return ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
#else
	return nullptr;
#endif
}

ULevelSequence* UVCamBlueprintFunctionLibrary::GetPendingTakeLevelSequence()
{
#if WITH_EDITOR
	const ITakeRecorderModule& TakeRecorderModule = FModuleManager::LoadModuleChecked<ITakeRecorderModule>("TakeRecorder");
	const UTakePreset* TakePreset = TakeRecorderModule.GetPendingTake();
	return TakePreset ? TakePreset->GetLevelSequence() : nullptr;
#else
	return nullptr;
#endif
}

bool UVCamBlueprintFunctionLibrary::OpenLevelSequence(ULevelSequence* LevelSequence)
{
#if WITH_EDITOR
	return ULevelSequenceEditorBlueprintLibrary::OpenLevelSequence(LevelSequence);
#else
	return false;
#endif
}

void UVCamBlueprintFunctionLibrary::PlayCurrentLevelSequence()
{
#if WITH_EDITOR
	ULevelSequenceEditorBlueprintLibrary::Play();
#endif
}

void UVCamBlueprintFunctionLibrary::PauseCurrentLevelSequence()
{
#if WITH_EDITOR
	ULevelSequenceEditorBlueprintLibrary::Pause();
#endif
}

void UVCamBlueprintFunctionLibrary::SetCurrentLevelSequenceCurrentFrame(int32 NewFrame)
{
#if WITH_EDITOR
	FMovieSceneSequencePlaybackParams Params(FFrameTime(NewFrame), EUpdatePositionMethod::Play);
	
	ULevelSequenceEditorBlueprintLibrary::SetGlobalPosition(Params);
#endif

}

int32 UVCamBlueprintFunctionLibrary::GetCurrentLevelSequenceCurrentFrame()
{
#if WITH_EDITOR
	FMovieSceneSequencePlaybackParams Position = ULevelSequenceEditorBlueprintLibrary::GetGlobalPosition();
	return Position.Frame.FloorToFrame().Value;
#else
	return 0;
#endif
}

int32 UVCamBlueprintFunctionLibrary::GetLevelSequenceLengthInFrames(const ULevelSequence* LevelSequence)
{
	if (LevelSequence)
	{
		int32 SequenceLowerBound = LevelSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue().Value;
		int32 SequenceUpperBound = LevelSequence->GetMovieScene()->GetPlaybackRange().GetUpperBoundValue().Value;
		int32 Length = SequenceUpperBound - SequenceLowerBound;
		return ConvertFrameTime(Length, LevelSequence->GetMovieScene()->GetTickResolution(), LevelSequence->GetMovieScene()->GetDisplayRate()).FloorToFrame().Value;
	}

	return 0;
}

int32 UVCamBlueprintFunctionLibrary::TimecodeToFrameAmount(FTimecode Timecode, const FFrameRate& InFrameRate)
{
	return Timecode.ToFrameNumber(InFrameRate).Value; 

}

FTimecode UVCamBlueprintFunctionLibrary::GetLevelSequenceFrameAsTimecode(const ULevelSequence* LevelSequence, int32 InFrame)
{
	if (LevelSequence)
	{
		return FTimecode::FromFrameNumber(InFrame, LevelSequence->GetMovieScene()->GetDisplayRate());
	}

	return FTimecode();
}

FTimecode UVCamBlueprintFunctionLibrary::GetLevelSequenceFrameAsTimecodeWithoutObject(const FFrameRate DisplayRate, int32 InFrame)
{
	return FTimecode::FromFrameNumber(InFrame, DisplayRate);
}

bool UVCamBlueprintFunctionLibrary::IsCurrentLevelSequencePlaying()
{
#if WITH_EDITOR
	return ULevelSequenceEditorBlueprintLibrary::IsPlaying();
#else
	return false;
#endif
}

UTexture* UVCamBlueprintFunctionLibrary::ImportSnapshotTexture(FString FileName, FString SubFolderName, FString AbsolutePathPackage)
{
#if WITH_EDITOR
	return UVPUtilitiesEditorBlueprintLibrary::ImportSnapshotTexture(FileName, SubFolderName, AbsolutePathPackage);
#else
	return nullptr;
#endif
}

bool UVCamBlueprintFunctionLibrary::EditorSaveAsset(FString AssetPath)
{
#if WITH_EDITOR
	return UEditorAssetLibrary::SaveAsset(AssetPath, true);
#else
	return false;
#endif
}

UObject* UVCamBlueprintFunctionLibrary::EditorLoadAsset(FString AssetPath)
{
#if WITH_EDITOR
	return UEditorAssetLibrary::LoadAsset(AssetPath);
#else
	return nullptr;
#endif
}

void UVCamBlueprintFunctionLibrary::ModifyObjectMetadataTags(UObject* InObject, FName InTag, FString InValue)
{
#if WITH_EDITOR
	if (InObject) 
	{
		UEditorAssetLibrary::SetMetadataTag(InObject, InTag, InValue); 

	}
#endif
}

TMap<FName, FString> UVCamBlueprintFunctionLibrary::GetObjectMetadataTags(UObject* InObject)
{

#if WITH_EDITOR
	if (InObject)
	{
		return UEditorAssetLibrary::GetMetadataTagValues(InObject); 
	}
#endif
return TMap<FName, FString>();

}

TArray<FAssetData> UVCamBlueprintFunctionLibrary::SortAssetsByTimecodeAssetData(TArray<FAssetData> LevelSequenceAssetData)
{
	TArray<FAssetData> SortedTimecodeArray = LevelSequenceAssetData;


	//Sort an array of AssetData by their timecode contained in TakeMetaData inteded for use with LevelSequences
	SortedTimecodeArray.Sort([](const FAssetData& LevelSequence1AssetData, const FAssetData& LevelSequence2AssetData)
	{
		FString LevelSequence1TimestampTagValue;
		bool bFoundLevelSequence1Tag = LevelSequence1AssetData.GetTagValue(UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_Timestamp(), LevelSequence1TimestampTagValue);

		FString LevelSequence2TimestampTagValue;
		bool bFoundLevelSequence2Tag = LevelSequence2AssetData.GetTagValue(UTakeMetaDataTagsFunctionLibrary::GetTakeMetaDataTag_Timestamp(), LevelSequence2TimestampTagValue);


		if (bFoundLevelSequence1Tag && bFoundLevelSequence2Tag)
		{
			FDateTime LevelSequence1TimeStamp;
			FDateTime LevelSequence2TimeStamp;
			FDateTime::Parse(LevelSequence1TimestampTagValue, LevelSequence1TimeStamp);
			FDateTime::Parse(LevelSequence2TimestampTagValue, LevelSequence2TimeStamp);
			return LevelSequence1TimeStamp > LevelSequence2TimeStamp;
		}

		//Handle cases in which valid metadata is found on one but not the another. 
		else if (bFoundLevelSequence1Tag == false && bFoundLevelSequence2Tag == true)
		{
			return false;
		}

		else if (bFoundLevelSequence1Tag == true && bFoundLevelSequence2Tag == false)
		{
			return true;

		}
		return false;

	});

	return SortedTimecodeArray;
}

void UVCamBlueprintFunctionLibrary::PilotActor(AActor* SelectedActor)
{
#if WITH_EDITOR
	if (SelectedActor)
	{
		ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();

		if (LevelEditorSubsystem)
		{
			LevelEditorSubsystem->PilotLevelActor(SelectedActor);

			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
			if (LevelEditor)
			{
				TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface();
				if (ActiveLevelViewport && !ActiveLevelViewport->IsLockedCameraViewEnabled())
				{
					ActiveLevelViewport->ToggleActorPilotCameraView();
				}
			}
		}
	}
#endif
}

bool UVCamBlueprintFunctionLibrary::UpdatePostProcessSettingsForCapture(USceneCaptureComponent2D* CaptureComponent, float DepthOfField, float FStopValue)
{
	if (CaptureComponent)
	{
		FPostProcessSettings NewCapturePostProcessSettings = CaptureComponent->PostProcessSettings;
		NewCapturePostProcessSettings.bOverride_DepthOfFieldFstop = true;
		NewCapturePostProcessSettings.bOverride_DepthOfFieldFocalDistance = true;

		NewCapturePostProcessSettings.DepthOfFieldFstop = FStopValue;
		NewCapturePostProcessSettings.DepthOfFieldFocalDistance = DepthOfField;

		CaptureComponent->PostProcessSettings = NewCapturePostProcessSettings;
		return true;
	}
	return false;
}

FFrameRate UVCamBlueprintFunctionLibrary::GetDisplayRate(ULevelSequence* LevelSequence)
{
	if (LevelSequence)
	{
		return LevelSequence->GetMovieScene()->GetDisplayRate();
	}
	return FFrameRate();
}

FFrameRate UVCamBlueprintFunctionLibrary::ConvertStringToFrameRate(FString InFrameRateString)
{
	TValueOrError<FFrameRate, FExpressionError> ParseResult = ParseFrameRate(*InFrameRateString);
	if (ParseResult.IsValid())
	{
		return ParseResult.GetValue();
	}
	else {
		return FFrameRate();
	}
}

bool UVCamBlueprintFunctionLibrary::CallFunctionByName(UObject* ObjPtr, FName FunctionName)
{
	if (ObjPtr)
	{
		if (UFunction* Function = ObjPtr->FindFunction(FunctionName))
		{
			ObjPtr->ProcessEvent(Function, nullptr);
			return true;
		}
		return false;
	}
	return false;
}

float UVCamBlueprintFunctionLibrary::CalculateAutoFocusDistance(FVector2D ReticlePosition, UCineCameraComponent* CineCamera)
{
	if (!CineCamera)
	{
		return 0.0f;
	}

	float MaxFocusTraceDistance = 1000000.0f;
	FVector TraceDirection;
	FVector CameraWorldLocation;
	if (!DeprojectScreenToWorld(ReticlePosition, CameraWorldLocation, TraceDirection))
	{
		return 0.0f;
	}

	FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(UpdateAutoFocus), true);

	const FVector TraceEnd = CameraWorldLocation + TraceDirection * MaxFocusTraceDistance;
	FHitResult Hit;
	const bool bHit = CineCamera->GetWorld()->LineTraceSingleByChannel(Hit, CameraWorldLocation, TraceEnd, ECC_Visibility, TraceParams);

	return (bHit) ? Hit.Distance : MaxFocusTraceDistance;
}


void UVCamBlueprintFunctionLibrary::EditorSetGameView(bool bIsToggled)
{
#if WITH_EDITOR
	//Only set Game view when streaming in editor mode (so not on PIE, SIE or standalone) 
	if (!GEditor || GEditor->IsPlaySessionInProgress())
	{
		return;
	}
	ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>();

	if (LevelEditorSubsystem)
	{
		return LevelEditorSubsystem->EditorSetGameView(bIsToggled);
	}
#endif
}

void UVCamBlueprintFunctionLibrary::EnableDebugFocusPlane(UCineCameraComponent* CineCamera, bool bEnabled)
{
#if WITH_EDITOR
	if (!CineCamera)
	{
		return;
	}
	CineCamera->FocusSettings.bDrawDebugFocusPlane = bEnabled;
#endif
}

FString UVCamBlueprintFunctionLibrary::GetNextUndoDescription()
{
#if WITH_EDITOR
	if (GEditor != nullptr && GEditor->Trans != nullptr)
	{
		return GEditor->Trans->GetUndoContext().Title.ToString();
	}
#endif

	return "";
}

bool UVCamBlueprintFunctionLibrary::CopyToCineCameraActor(UCineCameraComponent* SourceCameraComponent, ACineCameraActor* TargetCameraActor)
{
	if (!(IsValid(SourceCameraComponent) && IsValid(TargetCameraActor) && IsValid(TargetCameraActor->GetCineCameraComponent())))
	{
		return false;
	}

	// Ensure the root actor transforms match
	if (const USceneComponent* SourceAttachment = SourceCameraComponent->GetAttachParent())
	{
		TargetCameraActor->SetActorTransform(SourceAttachment->GetComponentTransform());
	}

	// Copy the properties from one CineCameraComponent to the other
	UCineCameraComponent* TargetCameraComponent = TargetCameraActor->GetCineCameraComponent();
	UEngine::CopyPropertiesForUnrelatedObjects(SourceCameraComponent, TargetCameraComponent);
	
	TargetCameraActor->UpdateComponentTransforms();

	return true;
}

void UVCamBlueprintFunctionLibrary::SetActorLabel(AActor* Actor, const FString& NewActorLabel)
{
#if WITH_EDITOR
	if (Actor)
	{
		Actor->SetActorLabel(NewActorLabel);
	}
#endif
}

bool UVCamBlueprintFunctionLibrary::IsTakeRecorderPanelOpen()
{
#if WITH_EDITOR
	return IsValid(UTakeRecorderBlueprintLibrary::GetTakeRecorderPanel());
#else
	return false;
#endif
}

bool UVCamBlueprintFunctionLibrary::TryOpenTakeRecorderPanel()
{
#if WITH_EDITOR
	return IsValid(UTakeRecorderBlueprintLibrary::OpenTakeRecorderPanel());
#else
	return false;
#endif
}

bool UVCamBlueprintFunctionLibrary::IsRecording()
{
#if WITH_EDITOR
	return UTakeRecorderBlueprintLibrary::IsRecording();
#else
	return false;
#endif
}

void UVCamBlueprintFunctionLibrary::SetOnTakeRecorderSlateChanged(FOnTakeRecorderSlateChanged_VCam OnTakeRecorderSlateChanged)
{
#if WITH_EDITOR
	FScriptDelegate Delegate;
	Delegate.BindUFunction(OnTakeRecorderSlateChanged.GetUObject(), OnTakeRecorderSlateChanged.GetFunctionName());
	UTakesCoreBlueprintLibrary::SetOnTakeRecorderSlateChanged(UTakesCoreBlueprintLibrary::FOnTakeRecorderSlateChanged(Delegate));
#endif
}

namespace UE::VirtualCamera::Private
{
#if WITH_EDITOR
	static bool DeprojectScreenToWorld(const TSharedPtr<SLevelViewport>& ActiveLevelViewport, const FVector2D& InScreenPosition, FVector& OutWorldPosition, FVector& OutWorldDirection)
	{
		if (ActiveLevelViewport.IsValid() && ActiveLevelViewport->GetActiveViewport())
		{
			FViewport* ActiveViewport = ActiveLevelViewport->GetActiveViewport();
			FLevelEditorViewportClient& LevelViewportClient = ActiveLevelViewport->GetLevelViewportClient();
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
				ActiveViewport,
				LevelViewportClient.GetScene(),
				LevelViewportClient.EngineShowFlags)
				.SetRealtimeUpdate(true));
			FSceneView* View = LevelViewportClient.CalcSceneView(&ViewFamily);

			const FIntPoint ViewportSize = ActiveViewport->GetSizeXY();
			const FIntRect ViewRect = FIntRect(0, 0, ViewportSize.X, ViewportSize.Y);
			const FMatrix InvViewProjectionMatrix = View->ViewMatrices.GetClipToWorld();
			FSceneView::DeprojectScreenToWorld(InScreenPosition, ViewRect, InvViewProjectionMatrix, OutWorldPosition, OutWorldDirection);
			return true;
		}
		return false;
	}
#endif
}

bool UVCamBlueprintFunctionLibrary::DeprojectScreenToWorld(const FVector2D& InScreenPosition, FVector& OutWorldPosition, FVector& OutWorldDirection)
{
	FName LevelEditorName(TEXT("LevelEditor"));
	bool bSuccess = false;

	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Game)
		{
			APlayerController* PC = Context.OwningGameInstance->GetFirstLocalPlayerController(Context.World());
			if (PC)
			{
				bSuccess |= PC->DeprojectScreenPositionToWorld(InScreenPosition.X, InScreenPosition.Y, OutWorldPosition, OutWorldDirection);
				break;
			}
		}
#if WITH_EDITOR
		else if (Context.WorldType == EWorldType::Editor)
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorName);
			const TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveLevelViewport();
			bSuccess = UE::VirtualCamera::Private::DeprojectScreenToWorld(ActiveLevelViewport, InScreenPosition, OutWorldPosition, OutWorldDirection);
		}
#endif
	}

	if (!bSuccess)
	{
		OutWorldPosition = FVector::ZeroVector;
		OutWorldDirection = FVector::ZeroVector;
	}
	return bSuccess;
}

bool UVCamBlueprintFunctionLibrary::DeprojectScreenToWorldByViewport(const FVector2D& InScreenPosition, EVCamTargetViewportID TargetViewport, FVector& OutWorldPosition, FVector& OutWorldDirection)
{
	bool bSuccess = false;
	
#if WITH_EDITOR
	const TSharedPtr<SLevelViewport> Viewport = UE::VCamCore::GetLevelViewport(TargetViewport);
	bSuccess = UE::VirtualCamera::Private::DeprojectScreenToWorld(Viewport, InScreenPosition, OutWorldPosition, OutWorldDirection);
#endif
	
	if (!bSuccess)
	{
		OutWorldPosition = FVector::ZeroVector;
		OutWorldDirection = FVector::ZeroVector;
	}
	return false;
}

namespace UE::VirtualCamera::Private
{
	enum class EBreakBehaviour : uint8
	{
		Break,
		Continue
	};

	/**
	 * @return SizeX, SizeY, and a SizeX x SizeY matrix where each element maps to a pixel.
	 * @see FViewport::GetHitProxyMap.
	 */
	static TTuple<int32, int32, TArray<HHitProxy*>> GetProxyMap(FViewport& Viewport, const FVector2D& InScreenPosition, const uint32 HitProxySize)
	{
		// See FViewport::GetHitProxy.
		// Compute a HitProxySize x HitProxySize test region with the center at (X,Y).
		int32 MinX = InScreenPosition.X - HitProxySize;
		int32 MinY = InScreenPosition.Y - HitProxySize;
		int32 MaxX = InScreenPosition.X + HitProxySize;
		int32 MaxY = InScreenPosition.Y + HitProxySize;
		
		// Clip the region to the viewport bounds.
		const FIntPoint ViewportSize = Viewport.GetSizeXY();
		MinX = FMath::Clamp(MinX, 0, ViewportSize.X - 1);
		MinY = FMath::Clamp(MinY, 0, ViewportSize.Y - 1);
		MaxX = FMath::Clamp(MaxX, 0, ViewportSize.X - 1);
		MaxY = FMath::Clamp(MaxY, 0, ViewportSize.Y - 1);

		int32 TestSizeX = MaxX - MinX + 1;
		int32 TestSizeY = MaxY - MinY + 1;
		TArray<HHitProxy*> Result;
		
		if (TestSizeX <= 0 || TestSizeY <= 0)
		{
			return { TestSizeX, TestSizeY, Result };
		}
		
		const FIntRect QueryRect(MinX, MinY, MaxX + 1, MaxY + 1);
		Viewport.InvalidateHitProxy();
		Viewport.GetHitProxyMap(QueryRect, Result);
		return { TestSizeX, TestSizeY, Result };
	}

	/** Enumerates all hit proxies that correspond to a component. */
	static void EnumerateHitProxies(
		FViewport& Viewport,
		const FVector2D& InScreenPosition,
		const uint32 HitProxySize,
		TFunctionRef<EBreakBehaviour(const UPrimitiveComponent& Component)> Callback
		)
	{
		auto[TestSizeX, TestSizeY, ProxyMap] = GetProxyMap(Viewport, InScreenPosition, HitProxySize);
		if (ProxyMap.IsEmpty())
		{
			return;
		}
		
		const auto ProcessProxy = [&Callback](HHitProxy* Proxy) -> EBreakBehaviour
		{
			if (Proxy && Proxy->IsA(HActor::StaticGetType()))
			{
				HActor* ActorProxy = static_cast<HActor*>(Proxy);
				const UPrimitiveComponent* Component = ActorProxy->PrimComponent;
				if (Component)
				{
					return Callback(*Component);
				}
			}

			return EBreakBehaviour::Continue;
		};

		// Process at the center first - if that it matches our requirements, it is our best option.
		const int32 CenterIndex = TestSizeY/2 * TestSizeX + TestSizeX/2;
		if (ProcessProxy(ProxyMap[CenterIndex]) == EBreakBehaviour::Break)
		{
			return;
		}
			
		for (int32 TestY = 0;TestY < TestSizeY; TestY++)
		{
			for (int32 TestX = 0;TestX < TestSizeX; TestX++)
			{
				HHitProxy* HitProxy = ProxyMap[TestY * TestSizeX + TestX];
				if (ProcessProxy(HitProxy) == EBreakBehaviour::Break)
				{
					return;
				}
			}
		}
	}

	static bool PassesFilters(const UPrimitiveComponent& Component, const FVCamTraceHitProxyQueryParams& Params)
	{
		return !Params.IgnoredActors.Contains(Component.GetOwner());
	}
}

bool UVCamBlueprintFunctionLibrary::MultiTraceHitProxyOnViewport(
	const FVector2D& InScreenPosition,
	EVCamTargetViewportID InTargetViewport,
	FVCamTraceHitProxyQueryParams InQueryParams,
	TArray<FVCamTraceHitProxyResult>& Result
	)
{
	// This is WITH_EDITOR because EVCamTargetViewportID only makes sense in editor builds.
	// Other than that, all the below functions actually exist in Runtime builds and would work.
#if WITH_EDITOR
	const TSharedPtr<SLevelViewport> ViewportWidget = UE::VCamCore::GetLevelViewport(InTargetViewport);
	const TSharedPtr<FSceneViewport> Viewport = ViewportWidget ? ViewportWidget->GetSceneViewport() : nullptr;
	if (!Viewport)
	{
		return false;
	}

	using namespace UE::VirtualCamera::Private;
	TArray<FVCamTraceHitProxyResult> HitResults;
	EnumerateHitProxies(*Viewport, InScreenPosition, FMath::Max(InQueryParams.HitProxySize, 0), [&InQueryParams, &HitResults](const UPrimitiveComponent& Component)
	{
		if (PassesFilters(Component, InQueryParams))
		{
			// Component is in a HActor proxy, which keeps a const reference. We cannot safely use const_cast without invoking undefined behavior.
			// Hence we must resort to this runtime search hack.
			TArray<UActorComponent*> Components = Component.GetOwner()->GetComponents().Array();
			UActorComponent** UnconstComponent = Components.FindByPredicate([&Component](const UActorComponent* OwnedComponent){ return &Component == OwnedComponent; });
			if (ensure(UnconstComponent))
			{
				UPrimitiveComponent* UnconstPrimitive = Cast<UPrimitiveComponent>(*UnconstComponent);
				HitResults.AddUnique({ UnconstPrimitive->GetOwner(), UnconstPrimitive });
			}
			
		}
		return EBreakBehaviour::Continue;
	});

	if (!HitResults.IsEmpty())
	{
		Result = MoveTemp(HitResults);
	}
	return !Result.IsEmpty();
#else
	return false;
#endif
}

TArray<UObject*> UVCamBlueprintFunctionLibrary::GetBoundObjects(FMovieSceneObjectBindingID CameraBindingID)
{
	TArray<UObject*> BoundObjectsArray;
#if WITH_EDITOR
	BoundObjectsArray = ULevelSequenceEditorBlueprintLibrary::GetBoundObjects(CameraBindingID);
#endif
	return BoundObjectsArray;
}

float UVCamBlueprintFunctionLibrary::GetPlaybackSpeed()
{
#if WITH_EDITOR
	TWeakPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer.IsValid())
	{
		return Sequencer.Pin()->GetPlaybackSpeed();
	}
#endif
	return 0.0f;
}

void UVCamBlueprintFunctionLibrary::SetPlaybackSpeed(float Value)
{
#if WITH_EDITOR
	TWeakPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer.IsValid())
	{
		Sequencer.Pin()->SetPlaybackSpeed(Value);
	}
#endif
}

bool UVCamBlueprintFunctionLibrary::IsAssetDirty(const FAssetData& AssetData)
{
	if (AssetData.IsAssetLoaded())
	{
		UPackage* Package = AssetData.GetPackage();
		if (Package && Package->IsDirty())
		{
			return true;
		}
	}
	return false;
}

void UVCamBlueprintFunctionLibrary::SetCameraOverscan(UCineCameraComponent* InCameraComponent, const FVector4f& InAsymmetricOverscan, bool bScaleResolution)
{
	if (InCameraComponent)
	{
		InCameraComponent->AsymmetricOverscan = InAsymmetricOverscan;
		InCameraComponent->bScaleResolutionWithOverscan = bScaleResolution;
	}
}

void UVCamBlueprintFunctionLibrary::ClearCameraOverscan(UCineCameraComponent* InCameraComponent)
{
	if (InCameraComponent)
	{
		InCameraComponent->AsymmetricOverscan = FVector4f::Zero();
		InCameraComponent->bScaleResolutionWithOverscan = false;
	}
}

#if WITH_EDITOR
TWeakPtr<ISequencer> UVCamBlueprintFunctionLibrary::GetSequencer()
{
	if (!GEditor)
	{
		return nullptr;
	}

	if (ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence())
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(LevelSequence, false);
		const ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		return LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	}
	return nullptr;
}
#endif