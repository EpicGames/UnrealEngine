// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequence/VCamTakesMetaDataMigration.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "HAL/ConsoleManager.h"
#include "LevelSequence.h"
#include "LevelSequence/VirtualCameraClipsMetaData.h"
#include "LevelSequenceShotMetaDataLibrary.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"

namespace UE::VirtualCamera::MigrationDetail
{
static TAutoConsoleVariable<bool> CVarAutoMigrateLevelSequenceOnAccess(
	TEXT("VirtualCamera.AutoMigrateLevelSequenceOnAccess"),
	true,
	TEXT("Whenever VirtualCamera opens a level sequence, whether to automatically migrate the meta data from the old UVirtualCameraClipsMetaData to the new UMovieSceneShotMetaData.")
);

static FAutoConsoleCommand CMigrateLevelSequenceCommand(
	TEXT("VirtualCamera.MigrateLevelSequences"),
	TEXT("Goes through all ULevelSequences in the project and migrates the UVirtualCameraClipsMetaData to UMovieSceneShotMetaData.\nWARNING: This may take a while."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
	   UVCamTakesMetaDataMigration::SLOW_MigrateAllVCamTakesMetaDataInProject();
	})
);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetIsNoGood(ULevelSequence* InLevelSequence, bool& bOutNoGood)
{
	if (ULevelSequenceShotMetaDataLibrary::GetIsNoGood(InLevelSequence, bOutNoGood))
	{
		return true;
	}

#if WITH_EDITOR
	const UVirtualCameraClipsMetaData* MetaData = InLevelSequence ? InLevelSequence->FindMetaData<UVirtualCameraClipsMetaData>() : nullptr;
	if (MetaData)
	{
		bOutNoGood = MetaData->GetIsNoGood();
		return true;
	}
#endif
	
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetIsFlagged(ULevelSequence* InLevelSequence, bool& bOutIsFlagged)
{
	if (ULevelSequenceShotMetaDataLibrary::GetIsFlagged(InLevelSequence, bOutIsFlagged))
	{
		return true;
	}

#if WITH_EDITOR
	const UVirtualCameraClipsMetaData* MetaData = InLevelSequence ? InLevelSequence->FindMetaData<UVirtualCameraClipsMetaData>() : nullptr;
	if (MetaData)
	{
		bOutIsFlagged = MetaData->GetIsFlagged();
		return true;
	}
#endif
	
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetFavoriteLevel(ULevelSequence* InLevelSequence, int32& OutFavoriteLevel)
{
	if (ULevelSequenceShotMetaDataLibrary::GetFavoriteRating(InLevelSequence, OutFavoriteLevel))
	{
		return true;
	}
	
#if WITH_EDITOR
	const UVirtualCameraClipsMetaData* MetaData = InLevelSequence ? InLevelSequence->FindMetaData<UVirtualCameraClipsMetaData>() : nullptr;
	if (MetaData)
	{
		OutFavoriteLevel = MetaData->GetFavoriteLevel();
		return true;
	}
#endif
	
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood)
{
	if (ULevelSequenceShotMetaDataLibrary::GetIsNoGoodByAssetData(InAssetData, bOutNoGood))
	{
		return true;
	}

	return InAssetData.GetTagValue(UVirtualCameraClipsMetaData::AssetRegistryTag_bIsNoGood, bOutNoGood);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged)
{
	if (ULevelSequenceShotMetaDataLibrary::GetIsFlaggedByAssetData(InAssetData, bOutIsFlagged))
	{
		return true;
	}

	return InAssetData.GetTagValue(UVirtualCameraClipsMetaData::AssetRegistryTag_bIsFlagged, bOutIsFlagged);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::GetFavoriteLevelByAssetData(const FAssetData& InAssetData, int32& OutFavoriteLevel)
{
	if (ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingByAssetData(InAssetData, OutFavoriteLevel))
	{
		return true;
	}

	return InAssetData.GetTagValue(UVirtualCameraClipsMetaData::AssetRegistryTag_FavoriteLevel, OutFavoriteLevel);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UVCamTakesMetaDataMigration::SetIsNoGood(ULevelSequence* InLevelSequence, bool bInIsNoGood)
{
	ULevelSequenceShotMetaDataLibrary::SetIsNoGood(InLevelSequence, bInIsNoGood);
}

void UVCamTakesMetaDataMigration::SetIsFlagged(ULevelSequence* InLevelSequence, bool bInIsFlagged)
{
	ULevelSequenceShotMetaDataLibrary::SetIsFlagged(InLevelSequence, bInIsFlagged);
}

void UVCamTakesMetaDataMigration::SetFavoriteLevel(ULevelSequence* InLevelSequence, int32 InFavoriteLevel)
{
	ULevelSequenceShotMetaDataLibrary::SetFavoriteRating(InLevelSequence, InFavoriteLevel);
}

bool UVCamTakesMetaDataMigration::NeedsToMigrateVCamMetaData(ULevelSequence* InLevelSequence)
{
#if WITH_EDITOR
	// UMovieSceneShotMetaData contains TOptional values.
	// UVirtualCameraClipsMetaData always contains values (not wrapped with TOptional).
	// MigrateVCamTakesMetaData always sets a value during migration. So if all tags have a set value, no migration needs to occur.
	return InLevelSequence
		&& InLevelSequence->FindMetaData<UVirtualCameraClipsMetaData>() != nullptr
		&& (!ULevelSequenceShotMetaDataLibrary::HasIsNoGood(InLevelSequence)
			|| !ULevelSequenceShotMetaDataLibrary::HasIsFlagged(InLevelSequence)
			|| !ULevelSequenceShotMetaDataLibrary::HasFavoriteRating(InLevelSequence));
#else
	return false;
#endif
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVCamTakesMetaDataMigration::NeedsToMigrateVCamMetaDataByAssetData(const FAssetData& InAssetData)
{
#if WITH_EDITOR
	// UMovieSceneShotMetaData contains TOptional values.
	// UVirtualCameraClipsMetaData always contains values (not wrapped with TOptional).
	// MigrateVCamTakesMetaData always sets a value during migration. So if all tags have a set value, no migration needs to occur.
	const bool bHasOldTags = InAssetData.FindTag(UVirtualCameraClipsMetaData::AssetRegistryTag_bIsFlagged)
		|| InAssetData.FindTag(UVirtualCameraClipsMetaData::AssetRegistryTag_bIsNoGood)
		|| InAssetData.FindTag(UVirtualCameraClipsMetaData::AssetRegistryTag_FavoriteLevel);
	const bool bHasMissingNewTags = !ULevelSequenceShotMetaDataLibrary::HasIsNoGoodByAssetData(InAssetData)
		|| !ULevelSequenceShotMetaDataLibrary::HasIsFlaggedByAssetData(InAssetData)
		|| !ULevelSequenceShotMetaDataLibrary::HasFavoriteRatingByAssetData(InAssetData);
	return bHasOldTags && bHasMissingNewTags;
#else
	return false;
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVCamTakesMetaDataMigration::MigrateVCamTakesMetaData(ULevelSequence* InLevelSequence)
{
#if WITH_EDITOR
	const UVirtualCameraClipsMetaData* OldMetaData = InLevelSequence ? InLevelSequence->FindMetaData<UVirtualCameraClipsMetaData>() : nullptr;
	const bool bNeedsMigration = OldMetaData != nullptr;
	if (!bNeedsMigration)
	{
		return;
	}

	// Don't overwrite values that may already have been written, e.g. by calling UVCamTakesMetaDataMigration::SetIsNoGood, etc.!
	if (!ULevelSequenceShotMetaDataLibrary::HasIsNoGood(InLevelSequence))
	{
		ULevelSequenceShotMetaDataLibrary::SetIsNoGood(InLevelSequence, OldMetaData->GetIsNoGood());
	}
	if (!ULevelSequenceShotMetaDataLibrary::HasIsFlagged(InLevelSequence))
	{
		ULevelSequenceShotMetaDataLibrary::SetIsFlagged(InLevelSequence, OldMetaData->GetIsFlagged());
	}
	if (!ULevelSequenceShotMetaDataLibrary::HasFavoriteRating(InLevelSequence))
	{
		ULevelSequenceShotMetaDataLibrary::SetFavoriteRating(InLevelSequence, OldMetaData->GetFavoriteLevel());
	}

	// Do not remove the meta data - we'll keep it around for now as backup.
	// UVirtualCameraClipsMetaData is set up to stop adding the asset tags if UMovieSceneMetaData is present.
	// InLevelSequence->RemoveMetaData<UVirtualCameraClipsMetaData>();
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UVCamTakesMetaDataMigration::GetAutoMigrateAccessedLevelSequencesCVar()
{
	return UE::VirtualCamera::MigrationDetail::CVarAutoMigrateLevelSequenceOnAccess.GetValueOnAnyThread();
}

void UVCamTakesMetaDataMigration::SLOW_MigrateAllVCamTakesMetaDataInProject()
{
	FScopedSlowTask SlowTask(10, NSLOCTEXT("VirtualCamera", "Migrate", "Migrating meta data for Level Sequences")); 
       
	TArray<FAssetData> LevelSequences;
	{
		FScopedSlowTask FilterSlowTask(1, NSLOCTEXT("VirtualCamera", "Migrate.Filtering", "Finding assets to migrate"));
		FARCompiledFilter Filter;
		Filter.ClassPaths.Add(FTopLevelAssetPath(ULevelSequence::StaticClass()));
		IAssetRegistry::Get()->GetAssets(Filter, LevelSequences);
	}

	FScopedSlowTask MigrationTask(LevelSequences.Num(), NSLOCTEXT("VirtualCamera", "Migrate.DoMigration", "Migrating meta data for Level Sequences")); 
	for (const FAssetData& AssetData : LevelSequences)
	{
		MigrationTask.EnterProgressFrame(1,
		   FText::Format(
			  NSLOCTEXT("VirtualCamera", "Migrate.MigrateAssetFmt", "Migrating {0}"),
			  FText::FromString(AssetData.GetSoftObjectPath().ToString())
		   )
		);

		if (!NeedsToMigrateVCamMetaDataByAssetData(AssetData))
		{
			continue;
		}
          
		ULevelSequence* Sequence = Cast<ULevelSequence>(AssetData.GetAsset());
		if (!Sequence)
		{
			continue;
		}

		Sequence->Modify(true);
		MigrateVCamTakesMetaData(Sequence);
       
		UPackage* Package = AssetData.GetPackage();
		Package->MarkPackageDirty();
		const FString FilePath = Package->GetLoadedPath().GetLocalFullPath();
		Package->Save(Package, Sequence, *FilePath, FSavePackageArgs());
	}
}