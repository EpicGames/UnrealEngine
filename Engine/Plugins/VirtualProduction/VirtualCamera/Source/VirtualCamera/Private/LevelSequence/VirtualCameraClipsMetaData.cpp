// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequence/VirtualCameraClipsMetaData.h"

#include "LevelSequence.h"
#include "LevelSequenceShotMetaDataLibrary.h"
#include "UObject/AssetRegistryTagsContext.h"

const FName UVirtualCameraClipsMetaData::AssetRegistryTag_FocalLength = "ClipsMetaData_FocalLength";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_bIsSelected = "ClipsMetaData_bIsSelected";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_RecordedLevelName = "ClipsMetaData_RecordedLevelName";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_FrameCountStart = "ClipsMetaData_FrameCountStart";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_FrameCountEnd = "ClipsMetaData_FrameCountEnd"; 
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_LengthInFrames = "ClipsMetaData_LengthInFrames";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_DisplayRate = "ClipsMetaData_DisplayRate";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_bIsACineCameraRecording = "ClipsMetaData_bIsACineCameraRecording";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_bIsNoGood = "ClipsMetaData_bIsNoGood";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_bIsFlagged = "ClipsMetaData_bIsFlagged";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_FavoriteLevel = "ClipsMetaData_FavoriteLevel";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_bIsCreatedFromVCam = "ClipsMetaData_bIsCreatedFromVCam";
const FName UVirtualCameraClipsMetaData::AssetRegistryTag_PostSmoothLevel = "ClipMetaData_PostSmoothLevel";

UVirtualCameraClipsMetaData::UVirtualCameraClipsMetaData(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_FocalLength()
{
	return AssetRegistryTag_FocalLength;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_IsSelected()
{
	return AssetRegistryTag_bIsSelected;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_RecordedLevel()
{
	return AssetRegistryTag_RecordedLevelName;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_FrameCountStart()
{
	return AssetRegistryTag_FrameCountStart;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_FrameCountEnd()
{
	return AssetRegistryTag_FrameCountEnd;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_LengthInFrames()
{
	return AssetRegistryTag_LengthInFrames;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_DisplayRate()
{
	return AssetRegistryTag_DisplayRate;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_IsCineACineCameraRecording()
{
	return AssetRegistryTag_bIsACineCameraRecording;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_IsNoGood()
{
	return AssetRegistryTag_bIsNoGood;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_IsFlagged()
{
	return AssetRegistryTag_bIsFlagged;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_FavoriteLevel()
{
	return AssetRegistryTag_FavoriteLevel;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_IsCreatedFromVCam()
{
	return AssetRegistryTag_bIsCreatedFromVCam;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FName UVirtualCameraClipsMetaData::GetClipsMetaDataTag_PostSmoothLevel()
{
	return AssetRegistryTag_PostSmoothLevel;
}

TSet<FName> UVirtualCameraClipsMetaData::GetAllClipsMetaDataTags()
{
	return {
		GetClipsMetaDataTag_PostSmoothLevel()
	};
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	IMovieSceneMetaDataInterface::ExtendAssetRegistryTags(Context);
	
	Context.AddTag(FAssetRegistryTag(AssetRegistryTag_PostSmoothLevel, FString::FromInt(PostSmoothLevel), FAssetRegistryTag::ETagType::TT_Numerical, FAssetRegistryTag::TD_None));

	// These may have been migrated
#if WITH_EDITOR
	const ULevelSequence* OwningSequence = Cast<const ULevelSequence>(Context.GetObject());
	if (!ULevelSequenceShotMetaDataLibrary::HasIsNoGood(OwningSequence))
	{
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_bIsNoGood, FString::FromInt(bIsNoGood), FAssetRegistryTag::ETagType::TT_Numerical, FAssetRegistryTag::TD_None));
	}
	if (!ULevelSequenceShotMetaDataLibrary::HasIsFlagged(OwningSequence))
	{
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_bIsFlagged, FString::FromInt(bIsFlagged), FAssetRegistryTag::ETagType::TT_Numerical, FAssetRegistryTag::TD_None));
	}
	if (!ULevelSequenceShotMetaDataLibrary::HasFavoriteRating(OwningSequence))
	{
		Context.AddTag(FAssetRegistryTag(AssetRegistryTag_FavoriteLevel, FString::FromInt(FavoriteLevel), FAssetRegistryTag::ETagType::TT_Alphabetical, FAssetRegistryTag::TD_None));
	}
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
void UVirtualCameraClipsMetaData::ExtendAssetRegistryTagMetaData(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UVirtualCameraClipsMetaData::GetFocalLength() const 
{
#if WITH_EDITOR
	return FocalLength;
#else
	return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVirtualCameraClipsMetaData::GetSelected() const
{
#if WITH_EDITOR
	return bIsSelected; 
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FString UVirtualCameraClipsMetaData::GetRecordedLevelName() const
{
#if WITH_EDITOR
	return RecordedLevelName; 
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int UVirtualCameraClipsMetaData::GetFrameCountStart() const
{
#if WITH_EDITOR
	return FrameCountStart; 
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int UVirtualCameraClipsMetaData::GetFrameCountEnd() const
{
#if WITH_EDITOR
	return FrameCountEnd; 
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int UVirtualCameraClipsMetaData::GetLengthInFrames()
{
#if WITH_EDITOR
	return LengthInFrames;
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FFrameRate UVirtualCameraClipsMetaData::GetDisplayRate()
{
#if WITH_EDITOR
	return DisplayRate;
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVirtualCameraClipsMetaData::GetIsACineCameraRecording() const
{
#if WITH_EDITOR
	return bIsACineCameraRecording; 
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVirtualCameraClipsMetaData::GetIsNoGood() const
{
#if WITH_EDITOR
	return bIsNoGood;
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UVirtualCameraClipsMetaData::GetIsFlagged() const
{
#if WITH_EDITOR
	return bIsFlagged;
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 UVirtualCameraClipsMetaData::GetFavoriteLevel() const
{
#if WITH_EDITOR
	return FavoriteLevel;
#else
return {};
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetFocalLength(float InFocalLength) 
{
#if WITH_EDITOR
	FocalLength = InFocalLength;
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetSelected(bool bInSelected)
{
#if WITH_EDITOR
	bIsSelected = bInSelected;
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetRecordedLevelName(FString InLevelName)
{
#if WITH_EDITOR
	RecordedLevelName = InLevelName;
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetFrameCountStart(int InFrame)
{
#if WITH_EDITOR
	FrameCountStart = InFrame; 
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetFrameCountEnd(int InFrame)
{
#if WITH_EDITOR
	FrameCountEnd = InFrame; 
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetLengthInFrames(int InLength)
{
#if WITH_EDITOR
	LengthInFrames = InLength; 
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetDisplayRate(FFrameRate InDisplayRate)
{
#if WITH_EDITOR
	DisplayRate = InDisplayRate; 
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UVirtualCameraClipsMetaData::SetIsACineCameraRecording(bool bInIsACineCameraRecording)
{
#if WITH_EDITOR
	bIsACineCameraRecording = bInIsACineCameraRecording;
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS