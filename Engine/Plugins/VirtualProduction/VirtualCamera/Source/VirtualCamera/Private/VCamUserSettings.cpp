// Copyright Epic Games, Inc. All Rights Reserved.

#include "VCamUserSettings.h"
#include "Misc/Paths.h"

UVirtualCameraUserSettings::UVirtualCameraUserSettings()
{
	PhotoSaveLocation.Path = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("VCamPhotos"));
	TextureSaveLocation.Path = TEXT("/Game/VCamPhotos");
	PhotoSaveMode = EVCamPhotoSaveMode::Both;
}