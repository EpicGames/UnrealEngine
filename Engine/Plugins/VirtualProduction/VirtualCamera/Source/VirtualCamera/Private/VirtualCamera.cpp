// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualCamera.h"

#include "AdvancedWidgetsModule.h"
#include "Misc/CoreDelegates.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "ITakeRecorderSourcesModule.h"
#include "VCamUserSettings.h"
#endif

DEFINE_LOG_CATEGORY(LogVirtualCamera);
#define LOCTEXT_NAMESPACE "FVirtualCameraModuleImpl"

namespace UE::VirtualCamera
{
	void FVirtualCameraModuleImpl::StartupModule()
	{
		// Loads widgets (ex. RadialSlider) that are potentially referenced by assets
		FModuleManager::Get().LoadModuleChecked<FAdvancedWidgetsModule>("AdvancedWidgets");

		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FVirtualCameraModuleImpl::OnPostInitEngine);
	}

	void FVirtualCameraModuleImpl::ShutdownModule()
	{
		UnregisterSettings();
	}

	void FVirtualCameraModuleImpl::OnPostInitEngine()
	{
		RegisterSettings();
	}

	void FVirtualCameraModuleImpl::RegisterSettings()
	{
#if WITH_EDITOR
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (ensure(SettingsModule))
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "VirtualCamera",
				LOCTEXT("VirtualCameraUserSettingsName", "Virtual Camera"),
				LOCTEXT("VirtualCameraUserSettingsDescription", "Configure the Virtual Camera settings."),
				GetMutableDefault<UVirtualCameraUserSettings>()
				);
		}
#endif
	}

	void FVirtualCameraModuleImpl::UnregisterSettings()
	{
#if WITH_EDITOR
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "VirtualCamera");
		}
#endif
	}
}

IMPLEMENT_MODULE(UE::VirtualCamera::FVirtualCameraModuleImpl, VirtualCamera)

#undef LOCTEXT_NAMESPACE