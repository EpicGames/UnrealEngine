// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVirtualCamera, Log, Log);

namespace UE::VirtualCamera
{
	class FVirtualCameraModuleImpl : public IModuleInterface
	{
	public:

		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

	private:

		void OnPostInitEngine();
		
		void RegisterSettings();
		void UnregisterSettings();
	};
}
