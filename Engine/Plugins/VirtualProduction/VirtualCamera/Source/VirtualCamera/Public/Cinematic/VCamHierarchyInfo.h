// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraComponent.h"
#include "GameFramework/Actor.h"
#include "VCamComponent.h"
#include "Misc/NotNull.h"

namespace UE::VirtualCamera
{
/** Gets components of a VCamActor for the purpose of deciding whether it should be recorded as cine camera in Sequencer. */
struct FVCamHierarchyInfo
{
	const AActor* Actor;
	UVCamComponent* VCamComponent;

	USceneComponent* RootComponent;
	UCineCameraComponent* Camera;

	explicit FVCamHierarchyInfo(TNotNull<const UObject*> SourceObject)
		: Actor(SourceObject->IsA<AActor>() ? Cast<AActor>(SourceObject) : SourceObject->GetTypedOuter<AActor>())
		, VCamComponent(Actor ? Actor->FindComponentByClass<UVCamComponent>() : nullptr)
		, RootComponent(Actor ? Actor->GetRootComponent() : nullptr)
		, Camera(VCamComponent ? VCamComponent->GetTargetCamera() : nullptr)
	{}

	bool ShouldRecordAsCineCamera() const { return VCamComponent != nullptr && Camera != nullptr && VCamComponent->GetRecordAsCineCamera(); }
};
}