// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Templates/Function.h"

#include "AssetFilteringAndSortingFunctionLibrary.generated.h"

struct FAssetData;

UENUM(BlueprintType)
enum class UE_DEPRECATED(5.5, "Use EEditorAssetSortOrder instead") ESortOrder : uint8
{
	Ascending,
	Descending
};

UENUM(BlueprintType)
enum class UE_DEPRECATED(5.5, "Use EEditorAssetMetaDataSortType instead") EAssetTagMetaDataSortType : uint8
{
	String,
	Numeric,
	DateTime
};

class UE_DEPRECATED(5.5, "Use the equivalent delegates for UEditorAssetSubsystem and UAssetRegistryHelpers instead") FAssetSortingPredicate;
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FAssetSortingPredicate, const FAssetData&, Left, const FAssetData&, Right);

/** This library's purpose is to facilitate Blueprints to discover assets using some filters and sort them. */
UCLASS(BlueprintType, Deprecated)
class UE_DEPRECATED(5.5, "Use the equivalent function in UEditorAssetSubsystem and UAssetRegistryHelpers instead") VIRTUALCAMERA_API UDEPRECATED_AssetFilteringAndSortingFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** Gets all assets which have the given tags. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips", meta = (DeprecatedFunction, DeprecationMessage = "Use EditorAssetSubsystem's GetAllAssetsByMetaDataTags instead."))
	UE_DEPRECATED(5.5, "Use UEditorAssetSubsystem::GetAllAssetsByMetaDataTags instead.")
	static TArray<FAssetData> GetAllAssetsByMetaDataTags(const TSet<FName>& RequiredTags, const TSet<UClass*>& AllowedClasses);

	/** Sorts the assets based on a custom Blueprint delegate.
	 * @param SortingPredicate Implements a Left <= Right relation
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips", meta = (DeprecatedFunction, DeprecationMessage = "Use AssetRegistryHelpers's SortByCustomPredicate instead."))
	UE_DEPRECATED(5.5, "Use UAssetRegistryHelpers::SortByCustomPredicate instead.")
	static void SortByCustomPredicate(UPARAM(Ref) TArray<FAssetData>& Assets, FAssetSortingPredicate SortingPredicate, ESortOrder SortOrder = ESortOrder::Ascending);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Sorts the assets by their asset name. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips", meta = (DeprecatedFunction, DeprecationMessage = "Use AssetRegistryHelpers's SortByAssetName instead."))
	UE_DEPRECATED(5.5, "Use UAssetRegistryHelpers::SortByAssetName instead.")
	static void SortByAssetName(UPARAM(Ref) TArray<FAssetData>& Assets, ESortOrder SortOrder = ESortOrder::Ascending);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * Sorts the assets based on their meta data's type.
	 * Supported types: FString, int, float, FDateTime.
	 * 
	 * @param Assets The assets to sort
	 * @param MetaDataTag The on which the sort is based
	 * @param SortOrder Whether to sort ascending or descending
	 * @return Whether it was possible to compare all the meta data successfully.
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips", meta = (DeprecatedFunction, DeprecationMessage = "Use EditorAssetSubsystem's SortByMetaData instead."))
	UE_DEPRECATED(5.5, "Use UEditorAssetSubsystem::SortByMetaData instead.")
	static bool SortByMetaData(UPARAM(Ref) TArray<FAssetData>& Assets, FName MetaDataTag, EAssetTagMetaDataSortType MetaDataType, ESortOrder SortOrder = ESortOrder::Ascending);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/**
	 * Util that does the actual sorting
	 * @param Predicate Implements a Left <= Right relation
	 */
	UE_DEPRECATED(5.5, "This function will not be replaced. Write your own version.")
	static void SortAssets(TArray<FAssetData>& Assets, TFunctionRef<bool(const FAssetData& Left, const FAssetData& Right)> Predicate, ESortOrder SortOrder);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
