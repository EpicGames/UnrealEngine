// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameViewFunctionLibrary.generated.h"

enum class EVCamTargetViewportID : uint8;

/** Library for changing the viewport into game view. Game view shows the scene as it appears in game. */
UCLASS()
class VIRTUALCAMERA_API UGameViewFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Toggles the game view.
	 * Game view shows the scene as it appears in game.
	 * 
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static void ToggleGameView(EVCamTargetViewportID ViewportID);

	/**
	 * @return Whether we can toggle game view. 
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static bool CanToggleGameView(EVCamTargetViewportID ViewportID);

	/**
	 * @return Whether we are in game view.
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static bool IsInGameView(EVCamTargetViewportID ViewportID);

	/**
	 * Sets whether the specified viewport should have the game view enabled.
	 * @note Only works in editor builds.
	 * */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static void SetGameViewEnabled(EVCamTargetViewportID ViewportID, bool bIsEnabled);

	/**
	 * Sets the game view mode for all open viewports. 
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static void SetGameViewEnabledForAllViewports(bool bIsEnabled);

	/**
	 * @return The state of each viewport's state. If the viewport is not currently open, then the returned map will not have any entry for it. 
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera | Game View")
	static TMap<EVCamTargetViewportID, bool> SnapshotGameViewStates();

	/**
	 * Sets the game view of the viewports as specified in the map. Util for using together with SnapshotGameViewStates. 
	 * @note Only works in editor builds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Game View")
	static void RestoreGameViewStates(const TMap<EVCamTargetViewportID, bool>& Snapshot);
};
