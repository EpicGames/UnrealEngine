// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectPtr.h"
#include "LevelSequenceVCamLibrary.generated.h"

class AActor;
class UCameraComponent;
class ULevelSequence;
class UObject;

USTRUCT(BlueprintType)
struct FPilotableSequenceCameraInfo
{
	GENERATED_BODY()

	/** A camera from a sequencer track */
	UPROPERTY(BlueprintReadOnly, Category = "Virtual Camera | Level Sequence")
	TObjectPtr<UCameraComponent> Camera;
};

/**
 * Utility functions for Level Sequences to implement VCamHUD UI.
 */
UCLASS()
class VIRTUALCAMERA_API ULevelSequenceVCamLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Checks whether this sequence has any camera cuts set up.
	 *
	 * If yes, you should pilot the sequence using ULevelSequenceEditorBlueprintLibrary::SetLockCameraCutToViewport instead of directly piloting
	 * by using FindLevelSequencePilotableCamerasInActiveLevelSequence.
	 */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera | Level Sequence", meta = (WorldContext = "WorldContextObject"))
	static bool HasAnyCameraCutsInLevelSequence(ULevelSequence* Sequence);

	/**
	 * Removes baked transform keys that fight a parent constraint on a VCam binding.
	 * Sets the section's TransformMask to match the constraint's TransformFilter so only
	 * constrained axes are evaluated. Must be called again if TransformFilter changes.
	 *
	 * @param Sequence    The level sequence to operate on.
	 * @param VCamActor   The VCam actor to clean up. Pass nullptr to process all bindings with constraint channels.
	 * @return            Number of sections cleared.
	 */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera | Level Sequence")
	static int32 ClearVCamConstraintBakeTracks(ULevelSequence* Sequence, AActor* VCamActor);

	/**
	 * Gets all cameras currently spawned by the active level sequence.
	 * Note: You must have called ULevelSequenceEditorBlueprintLibrary::OpenLevelSequence before calling this function.
	 * Note: Only works in the editor.
	 */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera | Level Sequence")
	static TArray<FPilotableSequenceCameraInfo> FindPilotableCamerasInActiveLevelSequence();
};
