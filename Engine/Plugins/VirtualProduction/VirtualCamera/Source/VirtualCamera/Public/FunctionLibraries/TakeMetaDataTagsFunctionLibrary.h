// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "TakeMetaDataTagsFunctionLibrary.generated.h"

/** This library's purpose is to expose the names of the UTakeMetaData asset registry tag names. */
UCLASS(BlueprintType)
class VIRTUALCAMERA_API UTakeMetaDataTagsFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/** The asset registry tag that contains the slate for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_Slate();

	/** The asset registry tag that contains the take number for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_TakeNumber();

	/** The asset registry tag that contains the timestamp for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_Timestamp();

	/** The asset registry tag that contains the timecode in for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_TimecodeIn();

	/** The asset registry tag that contains the timecode out for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_TimecodeOut();

	/** The asset registry tag that contains the user-description for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_Description();

	/** The asset registry tag that contains the level-path for this meta-data */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FName GetTakeMetaDataTag_LevelPath();

	/** Gets all tags for TakeMetaData */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static TSet<FName> GetAllTakeMetaDataTags();
};
