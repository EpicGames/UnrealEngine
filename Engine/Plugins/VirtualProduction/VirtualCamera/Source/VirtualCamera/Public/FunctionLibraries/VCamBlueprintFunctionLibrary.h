// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraComponent.h"
#include "AssetRegistry/AssetData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "MovieSceneObjectBindingID.h"
#include "VCamBlueprintFunctionLibrary.generated.h"

class AActor;
class ACineCameraActor;
class UCineCameraComponent;
class ULevelSequence;
class UPrimitiveComponent;
class USceneCaptureComponent2D;
class UVirtualCameraClipsMetaData;

#if WITH_EDITOR
class ISequencer;
#endif

enum class EVCamTargetViewportID : uint8;

USTRUCT(BlueprintType)
struct VIRTUALCAMERA_API FVCamTraceHitProxyQueryParams
{
	GENERATED_BODY()

	/** Determine the size of the query area around the center pixel. */
	UPROPERTY(EditAnywhere, BlueprintreadWrite, Category = "VirtualCamera")
	int32 HitProxySize = 5;

	/** Components on these actors should not be considered. */
	UPROPERTY(EditAnywhere, BlueprintreadWrite, Category = "VirtualCamera")
	TArray<TObjectPtr<AActor>> IgnoredActors;
};

USTRUCT(BlueprintType)
struct VIRTUALCAMERA_API FVCamTraceHitProxyResult
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "VirtualCamera")
	TWeakObjectPtr<AActor> HitActor;

	UPROPERTY(BlueprintReadOnly, Category = "VirtualCamera")
	TWeakObjectPtr<UPrimitiveComponent> HitComponent;

	friend bool operator==(const FVCamTraceHitProxyResult& Left, const FVCamTraceHitProxyResult& Right)
	{
		return Left.HitActor == Right.HitActor && Left.HitComponent == Right.HitComponent;
	}
	friend bool operator!=(const FVCamTraceHitProxyResult& Left, const FVCamTraceHitProxyResult& Right)
	{
		return !(Left == Right);
	}
};

UCLASS(BlueprintType)
class VIRTUALCAMERA_API UVCamBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Returns true if not in editor or if running the game in PIE or Simulate*/
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static bool IsGameRunning();

	/**
	 * Get the currently opened level sequence asset
	 * @see ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static ULevelSequence* GetCurrentLevelSequence();

	/**
	 * Gets the level sequence associated with the current pending take.
	 * @see ITakeRecorderModule::GetPendingTake
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static ULevelSequence* GetPendingTakeLevelSequence();

	/** Open a level sequence asset	 */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool OpenLevelSequence(ULevelSequence* LevelSequence);

	/** Play the current level sequence */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void PlayCurrentLevelSequence();

	/** Pause the current level sequence */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void PauseCurrentLevelSequence();

	/** Set playback position for the current level sequence in frames */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void SetCurrentLevelSequenceCurrentFrame(int32 NewFrame);

	/** Get the current playback position in frames */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static int32 GetCurrentLevelSequenceCurrentFrame();

	/** Get length in frames of a level sequence */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static int32 GetLevelSequenceLengthInFrames(const ULevelSequence* LevelSequence);

	/** Convert a frame from a level sequence to timecode */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static FTimecode GetLevelSequenceFrameAsTimecode(const ULevelSequence* LevelSequence, int32 InFrame);

	/** Convert a frame from a level sequence to timecode using only a provided display rate */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static FTimecode GetLevelSequenceFrameAsTimecodeWithoutObject(const FFrameRate DisplayRate, int32 InFrame);

	/** Check whether the sequence is actively playing. */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static bool IsCurrentLevelSequencePlaying();

	/** Imports image as a uasset */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static UTexture* ImportSnapshotTexture(FString FileName, FString SubFolderName, FString AbsolutePathPackage);
	
	/** Save an asset through path. Returns true on success. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips")
	static bool EditorSaveAsset(FString AssetPath);

	/** Load an asset through path. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips")
	static UObject* EditorLoadAsset(FString AssetPath);

	/** Modifies a UObject's metadata tags, adding a tag if the tag does not exist. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips")
	static void ModifyObjectMetadataTags(UObject* InObject, FName InTag, FString InValue);
	
	/** Retrieves UObject's metadata tags */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips")
	static TMap<FName, FString> GetObjectMetadataTags(UObject* InObject); 

	/** Sort array of FAssetData by metadata timecode **/
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Clips ")
	static TArray<FAssetData> SortAssetsByTimecodeAssetData(TArray<FAssetData> LevelSequenceAssets);
	
	/** Pilot the provided actor using editor scripting */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera | Streaming ")
	static void PilotActor(AActor* SelectedActor);

	/** Updates the provided USceneCaptureComponent2D's PostProcessingSettings. Returns true on success. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool UpdatePostProcessSettingsForCapture(USceneCaptureComponent2D* CaptureComponent, float DepthOfField, float FStopValue);

	/** Grab the display rate from a LevelSequences' MovieScene */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static FFrameRate GetDisplayRate(ULevelSequence* LevelSequence);

	/** Converts a double framerate to a FFrameRate */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static FFrameRate ConvertStringToFrameRate(FString InFrameRateString);

	/** Returns true if the function was found & executed correctly. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool CallFunctionByName(UObject* ObjPtr, FName FunctionName);

	/** Sets the current game view */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void EditorSetGameView(bool bIsToggled);

	/** Calculates auto focus */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static float CalculateAutoFocusDistance(FVector2D ReticlePosition, UCineCameraComponent* CineCamera);

	/** Get UObject from Camera Object Bindings*/
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID CameraBindingID);

	/** Enable/Disable debug focus plane*/
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void EnableDebugFocusPlane(UCineCameraComponent* CineCamera, bool bEnabled);

	/** Convert timecode to amount of frames at a given framerate */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static int32 TimecodeToFrameAmount(FTimecode Timecode, const FFrameRate& InFrameRate);

	/** Returns the description of the undo action that will be performed next.*/
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static FString GetNextUndoDescription();

	/** Copies all properties from a CineCameraComponent to a CineCameraActor and ensure the root actor transform is updated so the CameraComponents end up in the same World Space position */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool CopyToCineCameraActor(UCineCameraComponent* SourceCameraComponent, ACineCameraActor* TargetCameraActor);

	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void SetActorLabel(AActor* Actor, const FString& NewActorLabel);
	
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Take Recorder")
	static bool IsTakeRecorderPanelOpen();

	UFUNCTION(BlueprintCallable, Category = "VirtualCamera|Take Recorder")
	static bool TryOpenTakeRecorderPanel();

	/** Check whether a recording is currently active */
	UFUNCTION(BlueprintPure, Category="VirtualCamera|Take Recorder")
	static bool IsRecording();

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnTakeRecorderSlateChanged_VCam, const FString&, Slate);
	/** Called when the slate is changed. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera|Take Recorder")
	static void SetOnTakeRecorderSlateChanged(FOnTakeRecorderSlateChanged_VCam OnTakeRecorderSlateChanged);
	
	/* Get playback speed in Sequencer*/
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Sequencer")
	static float GetPlaybackSpeed();

	/* Set playback speed in Sequencer*/
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera|Sequencer")
	static void SetPlaybackSpeed(float Value=1.0);

	/* Convert 2D screen position to World Space 3D position and direction in the active viewport. Returns false if unable to determine value. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool DeprojectScreenToWorld(const FVector2D& InScreenPosition, FVector& OutWorldPosition, FVector& OutWorldDirection);

	/** Converts 2D screen position to World Space 3D position and direction in the specified viewport. Returns false if unable to determine value. Only works in editor builds. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool DeprojectScreenToWorldByViewport(const FVector2D& InScreenPosition, EVCamTargetViewportID TargetViewport, FVector& OutWorldPosition, FVector& OutWorldDirection);

	/**
	 * Traces from the viewport and returns all components that contribute to the pixels surrounding InScreenPosition.
	 * The size of the pixel area checked is controlled by InQueryParams.HitProxySize.
	 * 
	 * This finds actors that have NoCollision set. The actor is found by determining which actors contribute to the specified pixel.
	 * This function is designed for Editor builds; in Runtime builds, it returns false.
	 *
	 * @param InScreenPosition The viewport position to trace
	 * @param InTargetViewport The viewport to trace in
	 * @param InQueryParams Parameters for how the actors should be queried
	 * @param Result The result, set if this function returns true.
	 *
	 * @return Whether Result was written to
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static bool MultiTraceHitProxyOnViewport(const FVector2D& InScreenPosition, EVCamTargetViewportID InTargetViewport, FVCamTraceHitProxyQueryParams InQueryParams, TArray<FVCamTraceHitProxyResult>& Result);

	/* Checks if the asset is dirty */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera")
	static bool IsAssetDirty(const FAssetData& AssetData);

	/**
	 * Adds asymmetric overscan to the camera.
	 *
	 * @param InCameraComponent - The camera to add overscan to
	 * @param InAsymmetricOverscan - The amount of asymmetric overscan to apply, from 0.0 meaning no overscan to 1.0 meaning 100% overscan. By convention,
	 *								 X is the left overscan, Y is the right overscan, Z is the top overscan, and W is the bottom overscan.
	 *	@param bScaleResolution - Indicates if the render resolution should be scaled up so that the original frustum of the camera remains the same resolution
	 */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void SetCameraOverscan(UCineCameraComponent* InCameraComponent, const FVector4f& InAsymmetricOverscan, bool bScaleResolution = false);

	/** Clears any overscan from the camera */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void ClearCameraOverscan(UCineCameraComponent* InCameraComponent);
	
private:

#if WITH_EDITOR
	/** Returns the current sequencer. */
	static TWeakPtr<ISequencer> GetSequencer();
#endif
};
