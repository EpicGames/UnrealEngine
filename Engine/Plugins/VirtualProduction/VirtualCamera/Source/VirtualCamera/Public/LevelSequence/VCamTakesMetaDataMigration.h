// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "VCamTakesMetaDataMigration.generated.h"

class ULevelSequence;

/** Helps VCam Blueprints migrate. UVirtualCameraClipsMetaData */
UCLASS()
class UVCamTakesMetaDataMigration : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Gets whether the user has marked this take as no good.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Is No Good (Migration)"))
	static bool GetIsNoGood(ULevelSequence* InLevelSequence, bool& bOutNoGood);
	/**
	 * Gets whether this was flagged by a user.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Is Flagged (Migration)"))
	static bool GetIsFlagged(ULevelSequence* InLevelSequence, bool& bOutIsFlagged);
	/**
	 * Gets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Favorite Level (Migration)"))
	static bool GetFavoriteLevel(ULevelSequence* InLevelSequence, int32& OutFavoriteLevel);

	/**
	 * Gets whether the user has marked this take as no good.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Is No Good By Asset Data (Migration)"))
	static bool GetIsNoGoodByAssetData(const FAssetData& InAssetData, bool& bOutNoGood);
	/**
	 * Gets whether this was flagged by a user.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Is Flagged By Asset Data (Migration)"))
	static bool GetIsFlaggedByAssetData(const FAssetData& InAssetData, bool& bOutIsFlagged);
	/**
	 * Gets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated.
	 * Does no migration. Looks for new data source first and falls back to legacy data.
	 * @return Whether there is any data (legacy or new) stored.
	 */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Get Favorite Level By Asset Data (Migration)"))
	static bool GetFavoriteLevelByAssetData(const FAssetData& InAssetData, int32& OutFavoriteLevel);

	/** Sets whether the user has marked this take as no good. Always writes the data to the new data source. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera", meta = (DisplayName = "Set Is No Good (Migration)"))
	static void SetIsNoGood(ULevelSequence* InLevelSequence, bool bInIsNoGood);
	/** Sets whether this was flagged by a user. Always writes the data to the new data source. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera", meta = (DisplayName = "Set Is Flagged (Migration)"))
	static void SetIsFlagged(ULevelSequence* InLevelSequence, bool bInIsFlagged);
	/** Sets the favorite rating. The favorite rating is like a star rating, usually 1-3 if it was rated. Always writes the data to the new data source. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera", meta = (DisplayName = "Set Favorite Level (Migration)"))
	static void SetFavoriteLevel(ULevelSequence* InLevelSequence, int32 InFavoriteLevel);

	/** @return Whether the passed in level sequence contains the legacy UVirtualCameraClipsMetaData */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Needs Migration For VCam TakesMeta Data (Migration)"))
	static bool NeedsToMigrateVCamMetaData(ULevelSequence* InLevelSequence);
	/** @return Whether the passed in level sequence contains the legacy UVirtualCameraClipsMetaData */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera", meta = (DisplayName = "Needs Migration For VCam TakesMeta Data By Asset Data (Migration)"))
	static bool NeedsToMigrateVCamMetaDataByAssetData(const FAssetData& InAssetData);
	
	/** Migrates UVirtualCameraClipsMetaData to UMovieSceneShotMetaData. */
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera", meta = (DisplayName = "Migrate VCam Takes Meta Data (Migration)"))
	static void MigrateVCamTakesMetaData(ULevelSequence* InLevelSequence);

	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static bool GetAutoMigrateAccessedLevelSequencesCVar();
	
	UFUNCTION(BlueprintCallable, Category = "VirtualCamera")
	static void SLOW_MigrateAllVCamTakesMetaDataInProject();
};
