// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "IMovieSceneMetaData.h"
#include "Misc/FrameRate.h"
#include "VirtualCameraClipsMetaData.generated.h"

/**
 * Clips meta-data that is stored on ULevelSequence assets that are recorded through the virtual camera. 
 * Meta-data is retrieved through ULevelSequence::FindMetaData<UVirtualCameraClipsMetaData>().
 */
UCLASS(BlueprintType)
class UVirtualCameraClipsMetaData
	: public UObject
	, public IMovieSceneMetaDataInterface
{
public: 
	GENERATED_BODY()
	
	UVirtualCameraClipsMetaData(const FObjectInitializer& ObjInit);

public: 

	/** The asset registry tag that contains the focal length for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_FocalLength;

	/** The asset registry tag that contains if the selected state for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_bIsSelected;

	/** The asset registry tag that contains the recorded level name for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_RecordedLevelName;

	/** The asset registry tag that contains the FrameCountStart in for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_FrameCountStart;

	/** The asset registry tag that contains the FrameCountEnd out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_FrameCountEnd;

	/** The asset registry tag that contains the LengthInFrames out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_LengthInFrames; 

	/** The asset registry tag that contains the FrameCountEnd out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_DisplayRate;

	/** The asset registry tag that contains whether the clip was recorded with a CineCamera for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_bIsACineCameraRecording;
	
	/** The asset registry tag that contains whether this take is good or not */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsNoGoodAssetTag instead.")
	static const FName AssetRegistryTag_bIsNoGood;

	/** The asset registry tag that contains whether this was flagged by a user */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsFlaggedAssetTag instead.")
	static const FName AssetRegistryTag_bIsFlagged;

	/** The asset registry tag that contains its favorite status */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingAssetTag instead.")
	static const FName AssetRegistryTag_FavoriteLevel;

	/** The asset registry tag that contains whether it was created from a VCam */
	UE_DEPRECATED(5.6, "Data was removed.")
	static const FName AssetRegistryTag_bIsCreatedFromVCam;

	/* The asset registry tag that contains the post smooth level*/
	static const FName AssetRegistryTag_PostSmoothLevel;
	
public:

	/** The asset registry tag that contains the focal length for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_FocalLength();
	
	/** The asset registry tag that contains if the selected state for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_IsSelected();
	
	/** The asset registry tag that contains the recorded level name for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_RecordedLevel();
	
	/** The asset registry tag that contains the FrameCountStart in for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_FrameCountStart();
	
	/** The asset registry tag that contains the FrameCountEnd out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_FrameCountEnd();
	
	/** The asset registry tag that contains the LengthInFrames out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_LengthInFrames();
	
	/** The asset registry tag that contains the FrameCountEnd out for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_DisplayRate();
	
	/** The asset registry tag that contains whether the clip was recorded with a CineCamera for this meta-data */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "Virtual Camera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_IsCineACineCameraRecording();
	
	/** The asset registry tag that contains whether this take is good or not */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsNoGoodAssetTag instead.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Use ULevelSequenceShotMetaDataLibrary::GetIsNoGoodAssetTag instead."))
	static FName GetClipsMetaDataTag_IsNoGood();
	
	/** The asset registry tag that contains whether this was flagged by a user */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsFlaggedAssetTag instead.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Use ULevelSequenceShotMetaDataLibrary::GetIsFlaggedAssetTag instead."))
	static FName GetClipsMetaDataTag_IsFlagged();
	
	/** The asset registry tag that contains its favorite status */
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingAssetTag instead.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Use ULevelSequenceShotMetaDataLibrary::GetFavoriteRatingAssetTag instead."))
	static FName GetClipsMetaDataTag_FavoriteLevel();
	
	/** The asset registry tag that contains whether it was created from a VCam */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	static FName GetClipsMetaDataTag_IsCreatedFromVCam();

	/** The asset registry tag that contains post smooth level */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips")
	static FName GetClipsMetaDataTag_PostSmoothLevel();

	/** Gets all asset registry tags */
	UFUNCTION(BlueprintPure, Category = "VirtualCamera|Clips")
	static TSet<FName> GetAllClipsMetaDataTags();

public:

	/**
	* Extend the default ULevelSequence asset registry tags
	*/
	virtual void ExtendAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void ExtendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override {}
#if WITH_EDITOR
	virtual void ExtendAssetRegistryTagMetaData(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif

public:
	/**
	 * @return The focal length for this clip
	 */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	float GetFocalLength() const;

	/**
	* @return Whether or not the clip is selected.  
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	bool GetSelected() const;

	/**
	* @return The name of the clip's recorded level. 
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	FString GetRecordedLevelName() const;

	/**
	* @return The initial frame of the clip
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	int GetFrameCountStart() const;

	/**
	* @return The final frame of the clip
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	int GetFrameCountEnd() const;

	/**
	* @return The length in frames of the clip. 
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	int GetLengthInFrames();

	/**
	* @return The display rate of the clip. 
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	FFrameRate GetDisplayRate();

	/**
	* @return Whether the clip was recorded by a CineCameraActor
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	bool GetIsACineCameraRecording() const;

	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsNoGood or UVCamTakesMetaDataMigration instead")
	bool GetIsNoGood() const;
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetIsFlagged or UVCamTakesMetaDataMigration instead")
	bool GetIsFlagged() const;
	UE_DEPRECATED(5.6, "Use ULevelSequenceShotMetaDataLibrary::GetFavoriteRank or UVCamTakesMetaDataMigration instead")
	int32 GetFavoriteLevel() const;

public:
	/**
	* Set the focal length associated with this clip. 
	* @note: Used for tracking. Does not update the StreamedCameraComponent. 
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetFocalLength(float InFocalLength);
	
	/**
	* Set if this clip is 'selected'
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetSelected(bool bInSelected);

	/**
	* Set the name of the level that the clip was recorded in. 
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetRecordedLevelName(FString InLevelName);

	/**
	* Set the initial frame of the clip used for calculating duration.
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetFrameCountStart(int InFrame); 

	/**
	* Set the final frame of the clip used for calculating duration.
	*/
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetFrameCountEnd(int InFrame);

	/**
	 * Set the length in frames of the clip used for AssetData calculations. 
	 */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetLengthInFrames(int InLength);

	/**
	 * Set the DisplayRate of the clip used for AssetData calculations.
	 */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetDisplayRate(FFrameRate InDisplayRate);

	/**
	 * Set if the clip was recorded by a CineCameraActor
	 */
	UE_DEPRECATED(5.6, "Data was removed.")
	UFUNCTION(BlueprintCallable, Category = "Clips", meta=(DeprecatedFunction, DeprecationMessage="Data was removed."))
	void SetIsACineCameraRecording(bool bInIsACineCameraRecording);

private:

#if	WITH_EDITORONLY_DATA // These properties are deprecated
	UE_DEPRECATED(5.6, "Data was removed.")
	/** The focal length of the streamed camera used to record the take */
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	float FocalLength = 0.f;

	/** Whether or not the take was marked as 'selected' */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	bool bIsSelected = false; 

	/** The name of the level that the clip was recorded in */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	FString RecordedLevelName; 

	/** The initial frame of the clip used for calculating duration. */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	int FrameCountStart = 0;

	/** The last frame of the clip used for calculating duration. */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	int FrameCountEnd = 0; 

	/** The level sequence length in frames calculated from VirtualCameraSubsystem used for AssetData calculations */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	int LengthInFrames = 0; 

	/** The display rate of the level sequence used for AssetData calculations. */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	FFrameRate DisplayRate;

	/** If the LevelSequence was recorded with a CineCameraActor, rather than a VirtualCameraActor */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	bool bIsACineCameraRecording = false;
	
	/** Whether this take is marked as good */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Use ULevelSequenceShotMetaDataLibrary or UVCamTakesMetaDataMigration instead"))
	bool bIsNoGood = false;
	
	/** The asset registry tag that contains whether this was flagged by a user */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Use ULevelSequenceShotMetaDataLibrary or UVCamTakesMetaDataMigration instead"))
	bool bIsFlagged = false;

	/** The asset registry tag that contains its favorite status */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Use ULevelSequenceShotMetaDataLibrary or UVCamTakesMetaDataMigration instead"))
	int32 FavoriteLevel = 0;

	/** Whether the sequence was created from a VCam */
	UE_DEPRECATED(5.6, "Data was removed.")
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Data was removed."))
	bool bIsCreatedFromVCam = true;
#endif
	
	/* The asset registry tag that contains the post smooth level*/
	UPROPERTY(EditAnywhere, Category = "Clips", BlueprintReadWrite, meta = (AllowPrivateAccess = "true"))
	int32 PostSmoothLevel = 0;
};