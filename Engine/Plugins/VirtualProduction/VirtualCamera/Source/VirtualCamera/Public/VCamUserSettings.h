// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "VCamUserSettings.generated.h"

UENUM(BlueprintType)
enum class EVCamTutorialCompletionState : uint8
{
	/** Tutorial not completed */
	Pending,
	/** Tutorial was completed */
	Completed
};

UENUM(BlueprintType)
enum class EVCamPhotoSaveMode : uint8
{
	Off,
	PNG,
	Texture,
	Both
};

UCLASS(BlueprintType, Config = EditorPerProjectUserSettings)
class VIRTUALCAMERA_API UVirtualCameraUserSettings : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "Virtual Camera", meta = (Keywords = "Get Settings Virtual Camera VCam"))
	static UVirtualCameraUserSettings* GetSettings() { return GetMutableDefault<UVirtualCameraUserSettings>(); }

	UVirtualCameraUserSettings();

	/**
	 * Indicates whether the VCam tutorial is completed.
	 * You can manually reset this to Pending if you want to retake the tutorial.
	 * The tutorial shown in the default VCamHUD, e.g. to teach gestures.
	 */
	UPROPERTY(Config, BlueprintReadWrite, EditAnywhere, BlueprintSetter = "SetTutorialCompletionState", Category = "Virtual Camera")
	EVCamTutorialCompletionState VCamTutorialCompletionState = EVCamTutorialCompletionState::Pending;

	/** @return Whether the VCam tutorial has been completed. */
	UFUNCTION(BlueprintPure, Category = "Virtual Camera")
	bool IsTutorialCompleted() const { return VCamTutorialCompletionState == EVCamTutorialCompletionState::Completed; }

	/** Sets the tutorial completion state and saves it to the config file if it has changed. */
	UFUNCTION(BlueprintCallable, Category = "Virtual Camera")
	void SetTutorialCompletionState(EVCamTutorialCompletionState NewState)
	{
		if (VCamTutorialCompletionState != NewState)
		{
			VCamTutorialCompletionState = NewState;
			SaveConfig();
		}
	}

	/** VCam photo save mode*/
	UPROPERTY(Config, BlueprintReadWrite, EditAnywhere, Category = "VCam Photo")
	EVCamPhotoSaveMode PhotoSaveMode;

	/** Directory path to store vcam photos*/
	UPROPERTY(Config, BlueprintReadWrite, EditAnywhere, Category = "VCam Photo", meta = (EditCondition="PhotoSaveMode==EVCamPhotoSaveMode::PNG || PhotoSaveMode==EVCamPhotoSaveMode::Both"))
	FDirectoryPath PhotoSaveLocation;

	/** Directory path to store vcam photo texture assets*/
	UPROPERTY(Config, BlueprintReadWrite, EditAnywhere, Category = "VCam Photo", meta = (ContentDir, EditCondition="PhotoSaveMode==EVCamPhotoSaveMode::Texture || PhotoSaveMode==EVCamPhotoSaveMode::Both"))
	FDirectoryPath TextureSaveLocation;
};