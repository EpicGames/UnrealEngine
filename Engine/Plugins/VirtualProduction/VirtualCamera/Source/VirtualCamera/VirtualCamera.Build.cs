// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualCamera : ModuleRules
{
	public VirtualCamera(ReadOnlyTargetRules Target) : base(Target)
	{
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AugmentedReality",
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelSequence",
				"LiveLinkInterface",
				"MovieScene",
				"RemoteSession",
				"TimeManagement",
				"UMG",
				"VCamCore",
				"VPUtilities",
				"AssetRegistry",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"MediaIOCore",
				"Slate",
				"AdvancedWidgets",
				"MovieSceneTracks",
				"AnimationCore",  // FTransformFilter, FFilterOptionPerAxis - ClearVCamConstraintBakeTracks
				"Constraints",    // UTickableParentConstraint, FConstraintAndActiveChannel
			}
		);
		
		if (Target.bBuildEditor == true)
		{
			PublicDependencyModuleNames.AddRange(new string[]
			{
				"LevelSequenceEditor",
				"Sequencer",
				"SlateCore",
				"TakeRecorder"
			});
			
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"ConcertTakeRecorder",
				"EditorFramework",
				"EditorScriptingUtilities",
				"LevelEditor",
				"TakesCore",
				"TakeRecorderSources",
				"UnrealEd",
				"VPUtilitiesEditor",
			});
		}
	}
}
