// Copyright Epic Games, Inc. All Rights Reserved.

#include "VCamSupportForCinematicTooling.h"

#include "ITakeRecorderSourcesModule.h"
#include "Cinematic/VCamHierarchyInfo.h"

namespace UE::VirtualCamera
{
FVCamSupportForCinematicTooling::FVCamSupportForCinematicTooling()
{
	using namespace TakeRecorderSources;
	if (ITakeRecorderSourcesModule::IsAvailable())
	{
		ITakeRecorderSourcesModule::Get().RegisterCanRecordDelegate(
			TEXT("VirtualCamera_CanRecordDelegate"), FCanRecordDelegate::CreateStatic(&FVCamSupportForCinematicTooling::CanRecordComponent)
			);
	}
}

FVCamSupportForCinematicTooling::~FVCamSupportForCinematicTooling()
{
	using namespace TakeRecorderSources;
	if (ITakeRecorderSourcesModule::IsAvailable())
	{
		ITakeRecorderSourcesModule::Get().UnregisterCanRecordDelegate(TEXT("VirtualCamera_CanRecordDelegate"));
	}
}

bool FVCamSupportForCinematicTooling::CanRecordComponent(const TakeRecorderSources::FCanRecordArgs& InArgs)
{
	// If recording as cine camera, prevent the following components
	const FVCamHierarchyInfo VCamInfo(InArgs.ObjectToRecord);
	
	// Only record components that have equivalents on  ACineCameraActor, i.e. skip VCamComponent, InputComponent, CineCaptureComponent2D, etc.
	const bool bIsComponentFromCineCameraActor = InArgs.ObjectToRecord == VCamInfo.RootComponent || InArgs.ObjectToRecord == VCamInfo.Camera;
	// If VCamActor is not configured to record as cine camera, allow all components.
	const bool bAllowAllComponents = !VCamInfo.ShouldRecordAsCineCamera();
	
	return bAllowAllComponents || bIsComponentFromCineCameraActor || InArgs.ObjectToRecord == VCamInfo.Actor;
}
}
