// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"

namespace UE::TakeRecorderSources { struct FCanRecordArgs; }

namespace UE::VirtualCamera
{
/** Manages setting up global delegates, etc. so VCam operates correctly with Take Recorder & Sequencer. */
class FVCamSupportForCinematicTooling : public FNoncopyable
{
public:

	FVCamSupportForCinematicTooling();
	~FVCamSupportForCinematicTooling();

	/** Used by ITakeRecorderSourcesModule::RegisterCanRecordDelegate. If a VCam is set to record as ACineCameraActor, this will skip any additional components, such as VCam and the input component. */
	static bool CanRecordComponent(const TakeRecorderSources::FCanRecordArgs& InArgs);
};
}

