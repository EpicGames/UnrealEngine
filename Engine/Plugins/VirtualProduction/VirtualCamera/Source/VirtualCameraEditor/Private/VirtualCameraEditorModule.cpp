// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "ActorFactories/ActorFactoryBlueprint.h"
#include "CineCameraActor.h"
#include "Filters/CustomClassFilterData.h"
#include "IPlacementModeModule.h"
#include "IVPUtilitiesEditorModule.h"
#include "LevelEditor.h"
#include "LevelEditorOutlinerSettings.h"
#include "Cinematic/VCamSupportForCinematicTooling.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "FVirtualCameraEditorModule"

DEFINE_LOG_CATEGORY_STATIC(LogVirtualCameraEditor, Log, All);

namespace UE::VirtualCamera
{

class FVirtualCameraEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		RegisterPlacementModeItems();
		RegisterOutlinerFilters();

		CinematicToolingSupport = MakeUnique<FVCamSupportForCinematicTooling>();
	}

	virtual void ShutdownModule() override
	{
		CinematicToolingSupport.Reset();
	}

private:
	
	/** Manages setting up global delegates, etc. so VCam operates correctly with Take Recorder & Sequencer. */
	TUniquePtr<FVCamSupportForCinematicTooling> CinematicToolingSupport;
	
	void RegisterPlacementModeItems()
	{
		if (const FPlacementCategoryInfo* Info = IVPUtilitiesEditorModule::Get().GetVirtualProductionPlacementCategoryInfo()
			; Info && GEditor)
		{
			
			FAssetData VirtualCamera2ActorAssetData(
				TEXT("/VirtualCamera/VCamActor"),
				TEXT("/VirtualCamera"),
				TEXT("VCamActor"),
				FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"))
			);

			// Makes it appear in the VP category ...
			IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShared<FPlaceableItem>(
	              *UActorFactoryBlueprint::StaticClass(),
	              VirtualCamera2ActorAssetData,
	              FName("ClassThumbnail.CameraActor"),
	              FName("ClassIcon.CameraActor"),
	              TOptional<FLinearColor>(),
	              TOptional<int32>(),
	              LOCTEXT("VCam Actor", "VCam Actor")
	          )
			);

			// ... but if you search for it by text this is needed to make it show up (without having the user load it manually).
			// The search filters everything in the FBuiltInPlacementCategories::AllClasses category only;
			// it contains 1. loaded BP classes and 2. specialized actor factories. This manual load adds it to case 1.
			LoadAssetAsync(
				FTopLevelAssetPath(TEXT("/VirtualCamera/VCamActor"), TEXT("VCamActor_C")),
				FLoadAssetAsyncDelegate::CreateLambda([](const FTopLevelAssetPath& /*AssetPath*/, UObject* LoadedObject, EAsyncLoadingResult::Type)
				{
					UE_CLOGF(LoadedObject == nullptr, LogVirtualCameraEditor, Warning, "Failed to load '/VirtualCamera/VCamActor.VCamActor_C'. Has the Blueprint been moved?");
				}));
		}
	}

	void RegisterOutlinerFilters()
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
		{
			if (const TSharedPtr<FFilterCategory> VPFilterCategory = LevelEditorModule->GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::VirtualProduction()))
			{
				TSharedRef<FCustomClassFilterData> CineCameraActorClassData =
					MakeShared<FCustomClassFilterData>(ACineCameraActor::StaticClass(), VPFilterCategory, FLinearColor::White);
				LevelEditorModule->AddCustomClassFilterToOutliner(CineCameraActorClassData);
			}
		}
	}
};

IMPLEMENT_MODULE(FVirtualCameraEditorModule, VirtualCameraEditor)
}

#undef LOCTEXT_NAMESPACE