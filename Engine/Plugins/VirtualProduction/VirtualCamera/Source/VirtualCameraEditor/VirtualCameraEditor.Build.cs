// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VirtualCameraEditor : ModuleRules
{
	public VirtualCameraEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"CinematicCamera",
				"EditorWidgets",
				"Engine",
				"PlacementMode", 
				"Settings",
				"UnrealEd",
				"TakeRecorderSources",
				"VCamCore",
				"VirtualCamera", 
				"VPUtilitiesEditor", 
			}
		);
	}
}
