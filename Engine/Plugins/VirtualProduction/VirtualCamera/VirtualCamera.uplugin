{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "VirtualCamera",
	"Description": "Content for VirtualCameraCore which adds actors, components, and utilities for controlling and viewing cameras via physical devices.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "VCamExtensions",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VCamExtensionsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VirtualCamera",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "VirtualCameraEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name" : "CineCameraRigs",
			"Enabled": true
		},
		{
			"Name" : "MultiUserTakes",
			"Enabled": true
		},
		{
			"Name" : "VirtualCameraCore",
			"Enabled": true
		},
		{
			"Name": "CineCameraSceneCapture",
			"Enabled": true
		},
		{
			"Name": "BlueprintFileUtils",
			"Enabled": true
		}
	]
}
